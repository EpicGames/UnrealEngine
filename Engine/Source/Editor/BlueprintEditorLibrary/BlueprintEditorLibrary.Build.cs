// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class BlueprintEditorLibrary : ModuleRules
{
	public BlueprintEditorLibrary(ReadOnlyTargetRules Target) : base(Target)
	{		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"BlueprintGraph"
			}
		);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimGraph",
				"CoreUObject",
				"Engine",
				"Json",
				"JsonUtilities",
				"KismetCompiler",
				"UnrealEd",
			}
		);
	}
}
