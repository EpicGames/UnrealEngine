// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEditorLibrary.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "BlueprintEditorLibrary/BlueprintGraphPin.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditor.h"
#include "AnimGraphNode_Base.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintTypePromotion.h"
#include "Components/TimelineComponent.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/SimpleConstructionScript.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_Event.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_CommutativeAssociativeBinaryOperator.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_PromotableOperator.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "KismetCompilerModule.h"
#include "Logging/LogCategory.h"
#include "Logging/StructuredLog.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EngineVersion.h"
#include "ObjectEditorUtils.h"
#include "PackageTools.h"
#include "ScopedTransaction.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintEditorLibrary)

#define LOCTEXT_NAMESPACE "BlueprintEditorLibrary"

DEFINE_LOG_CATEGORY(LogBlueprintEditorLib);

///////////////////////////////////////////////////////////
// InternalBlueprintEditorLibrary

namespace InternalBlueprintEditorLibrary
{
	/**
	* Replace the OldNode with the NewNode and reconnect it's pins. If the pins don't
	* exist on the NewNode, then orphan the connections.
	*
	* @param OldNode		The old node to replace
	* @param NewNode		The new node to put in the old node's place
	*/
	static bool ReplaceOldNodeWithNew(UEdGraphNode* OldNode, UEdGraphNode* NewNode)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		
		bool bSuccess = false;

		if (Schema && OldNode && NewNode)
		{
			TMap<FName, FName> OldToNewPinMap;
			for (UEdGraphPin* Pin : OldNode->Pins)
			{
				if (Pin->ParentPin != nullptr)
				{
					// ReplaceOldNodeWithNew() will take care of mapping split pins (as long as the parents are properly mapped)
					continue;
				}
				else if (Pin->PinName == UEdGraphSchema_K2::PN_Self)
				{
					// there's no analogous pin, signal that we're expecting this
					OldToNewPinMap.Add(Pin->PinName, NAME_None);
				}
				else
				{
					// The input pins follow the same naming scheme
					OldToNewPinMap.Add(Pin->PinName, Pin->PinName);
				}
			}
			
			bSuccess = Schema->ReplaceOldNodeWithNew(OldNode, NewNode, OldToNewPinMap);
			// reconstructing the node will clean up any
			// incorrect default values that may have been copied over
			NewNode->ReconstructNode();			
		}

		return bSuccess;
	}

	/**
	* Returns true if any of these nodes pins have any links. Does not check for 
	* a default value on pins
	*
	* @param Node		The node to check
	*
	* @return bool		True if the node has any links, false otherwise.
	*/
	static bool NodeHasAnyConnections(const UEdGraphNode* Node)
	{
		if (Node)
		{
			for (const UEdGraphPin* Pin : Node->Pins)
			{
				if (Pin && Pin->LinkedTo.Num() > 0)
				{
					return true;
				}
			}
		}

		return false;
	}

	/**
	* Attempt to close any open editors that may be relevant to this blueprint. This will prevent any 
	* problems where the user could see a previously deleted node/graph.
	*
	* @param Blueprint		The blueprint that is being edited
	*/
	static void CloseOpenEditors(UBlueprint* Blueprint)
	{
		UAssetEditorSubsystem* AssetSubsystem = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;
		if (AssetSubsystem && Blueprint)
		{
			AssetSubsystem->CloseAllEditorsForAsset(Blueprint);
		}
	}

	/**
	 * Converts the given EVersionComparison to the BP friendly EAssetSaveVersionComparisonResults type.
	 */
	static EAssetSaveVersionComparisonResults ConvertSaveVersionComparison(const EVersionComparison RawVersionCompare)
	{
		EAssetSaveVersionComparisonResults Result = EAssetSaveVersionComparisonResults::InvalidComparison;
	
		switch (RawVersionCompare)
		{
		case EVersionComparison::Neither:
			Result = EAssetSaveVersionComparisonResults::Identical;
			break;
		case EVersionComparison::First:
			Result = EAssetSaveVersionComparisonResults::Newer;
			break;
		case EVersionComparison::Second:
			Result = EAssetSaveVersionComparisonResults::Older;
			break;
		}

		return Result;
	}

	// Iterator helper for walking a class hierarchy's member variables, whether they are native or not - return false to break the loop:
	void ForEachInheritedVariable(const UBlueprint* BP, TFunctionRef<bool(const UClass*, const FProperty*, const FBPVariableDescription*)> Ftor);
	bool GetVariableType(const FProperty* P, const FBPVariableDescription* D, FEdGraphPinType& OutType);
	FString GetVariableFullName(const UClass* C, const FProperty* P, const FBPVariableDescription* D);
	FString GetVariableShortName(const FProperty* P, const FBPVariableDescription* D);
};

void InternalBlueprintEditorLibrary::ForEachInheritedVariable(const UBlueprint* BP, TFunctionRef<bool(const UClass*, const FProperty*, const FBPVariableDescription*)> Ftor)
{
	check(BP);
	UClass* Class = BP->ParentClass;
	while(Class)
	{
		const FString ClassPath = Class->GetPathName();
		if (const UBlueprint* ParentBP = Cast<UBlueprint>(Class->ClassGeneratedBy))
		{
			for (const FBPVariableDescription& VarDesc : ParentBP->NewVariables)
			{
				if(!Ftor(Class, nullptr, &VarDesc))
				{
					return;
				}
			}
			Class = ParentBP->ParentClass;
		}
		else
		{
			for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
			{
				if(!Ftor(Class, *PropertyIt, nullptr))
				{
					return;
				}
			}
			Class = Class->GetSuperClass();
		}
	}
}

bool InternalBlueprintEditorLibrary::GetVariableType(const FProperty* P, const FBPVariableDescription* D, FEdGraphPinType& OutPinType)
{
	check(P || D);
	if(D)
	{
		OutPinType = D->VarType;
		return true;
	}

	FEdGraphPinType PinType;
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	return Schema->ConvertPropertyToPinType(P, OutPinType);
}

FString InternalBlueprintEditorLibrary::GetVariableFullName(const UClass* C, const FProperty* P, const FBPVariableDescription* D)
{
	check(C);
	check(P || D);
	const FString ClassPath = C->GetPathName();
	if(D)
	{
		return FString::Printf(TEXT("%s.%s"), *ClassPath, *D->VarName.ToString());
	}

	return FString::Printf(TEXT("%s.%s"), *ClassPath, *P->GetName());
}

FString InternalBlueprintEditorLibrary::GetVariableShortName(const FProperty* P, const FBPVariableDescription* D)
{
	check(P || D);
	if(D)
	{
		return D->VarName.ToString();
	}
	return P->GetName();
}

///////////////////////////////////////////////////////////
// UBlueprintEditorLibrary

UBlueprintEditorLibrary::UBlueprintEditorLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UBlueprintEditorLibrary::ReplaceVariableReferences(UBlueprint* Blueprint, const FName OldVarName, const FName NewVarName)
{
	if (!Blueprint || OldVarName.IsNone() || NewVarName.IsNone())
	{
		return;
	}

	FBlueprintEditorUtils::RenameVariableReferences(Blueprint, Blueprint->GeneratedClass, OldVarName, NewVarName);
}

UEdGraph* UBlueprintEditorLibrary::FindEventGraph(UBlueprint* Blueprint)
{
	return Blueprint ? FBlueprintEditorUtils::FindEventGraph(Blueprint) : nullptr;
}

void UBlueprintEditorLibrary::CompareAssetSaveVersionTo(const UObject* Asset, const FString& VersionToCheckString, EAssetSaveVersionComparisonResults& Result)
{
	Result = EAssetSaveVersionComparisonResults::InvalidComparison;
	
	if (!Asset)
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] 'Asset' is null! Cannot compare to engine version '%ls'", __func__, *VersionToCheckString);
		return;
	}

	FEngineVersion VersionToCheck = {};
	const bool bSuccessfulParse = FEngineVersion::Parse(VersionToCheckString, OUT VersionToCheck);
	if (!bSuccessfulParse)
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] 'VersionToCheckString' value of '%ls' is not a valid FEngineVersion!", __func__, *VersionToCheckString);
		return;
	}
	
	// The linker has the data about what engine version was used to save this asset
	FLinkerLoad* Linker = Asset->GetLinker();
	if (!Linker)
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] Failed to find the linker for asset '%ls'", __func__, *GetNameSafe(Asset));
		return;
	}

	const FEngineVersion& AssetVersion = Linker->Summary.SavedByEngineVersion;
	
	EVersionComponent* DifferingComponent =  nullptr;
	const EVersionComparison Comparison = FEngineVersionBase::GetNewest(AssetVersion, VersionToCheck, DifferingComponent);

	Result = InternalBlueprintEditorLibrary::ConvertSaveVersionComparison(Comparison);
}

void UBlueprintEditorLibrary::CompareSoftObjectSaveVersionTo(const TSoftObjectPtr<UObject> ObjectToCheck, const FString& VersionToCheckString, EAssetSaveVersionComparisonResults& Result)
{
	Result = EAssetSaveVersionComparisonResults::InvalidComparison;

	if (!ObjectToCheck.IsValid())
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] An invalid ObjectToCheck has been provided, cannot compare save versions.", __func__);
		return;
	}
	
	FString AbsolutePackageFilePath;
	const bool bSuccessfulyFoundFile = FPackageName::DoesPackageExist(ObjectToCheck.GetLongPackageName(), OUT &AbsolutePackageFilePath);
	if (!bSuccessfulyFoundFile)
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] Failed to find package on disk for soft object '%ls'", __func__, *ObjectToCheck.ToString());
		return;
	}
	
	// Ensure that this is indeed a package file path. This should always be true if the above DoesPackageExist function works.
	if (!ensure(FPackageName::IsPackageFilename(AbsolutePackageFilePath)))
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] '%ls' is not a package file path! (FPackageName::IsPackageFilename returned false)", __func__, *AbsolutePackageFilePath);
		return;
	}

	// Make sure we have a valid version to compare to before attempting to open a file reader
	FEngineVersion VersionToCheck = {};
	const bool bSuccessfulParse = FEngineVersion::Parse(VersionToCheckString, OUT VersionToCheck);
	if (!bSuccessfulParse)
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] 'VersionToCheckString' value of '%ls' is not a valid FEngineVersion!", __func__, *VersionToCheckString);
		return;
	}
	
	// Create a file reader to load the file and read its package summary data, which has the save version
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*AbsolutePackageFilePath));
	if (!FileReader.IsValid())
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] Failed to open file reader for path '%ls'", __func__, *AbsolutePackageFilePath);
		return;
	}
	
	FPackageFileSummary FileSummary;
	(*FileReader) << FileSummary;

	// Make sure this is indeed a package
	if (FileSummary.Tag == PACKAGE_FILE_TAG)
	{
		EVersionComponent* DifferingComponent =  nullptr;
		const EVersionComparison Comparison = FEngineVersionBase::GetNewest(FileSummary.SavedByEngineVersion, VersionToCheck, DifferingComponent);
		Result = InternalBlueprintEditorLibrary::ConvertSaveVersionComparison(Comparison);
	}

	// Clean up our file reader, we are done with it
	FileReader->Close();
	FileReader.Reset();
}

FString UBlueprintEditorLibrary::GetSavedByEngineVersion(const UObject* Asset)
{
	static const FString InvalidVersion = TEXT("INVALID");
	if (!Asset)
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] 'Asset' is null! Cannot get the 'saved by' engine version", __func__);
		return InvalidVersion;
	}

	// The linker has the data about what engine version was used to save this asset
	FLinkerLoad* Linker = Asset->GetLinker();
	if (!Linker)
	{
		UE_LOGF(LogBlueprintEditorLib, Error, "[%s] Failed to find the linker for asset '%ls'", __func__, *GetNameSafe(Asset));
		return InvalidVersion;
	}
	
	const FEngineVersion& AssetSavedVersion = Linker->Summary.SavedByEngineVersion;
	
	return AssetSavedVersion.ToString();
}

FString UBlueprintEditorLibrary::GetCurrentEngineVersion()
{
	return FEngineVersion::Current().ToString();
}

UEdGraph* UBlueprintEditorLibrary::FindGraph(UBlueprint* Blueprint, FName GraphName)
{
	if (Blueprint && !GraphName.IsNone())
	{
		TArray<UEdGraph*> AllGraphs;
		Blueprint->GetAllGraphs(AllGraphs);

		for (UEdGraph* CurrentGraph : AllGraphs)
		{
			if (CurrentGraph->GetFName() == GraphName)
			{
				return CurrentGraph;
			}
		}
	}

	return nullptr;
}

TArray<UEdGraph*> UBlueprintEditorLibrary::ListGraphs(UBlueprint* Blueprint)
{
	TArray<UEdGraph*> Graphs;
	if (Blueprint)
	{
		TArray<UEdGraph*> AllGraphs;
		Blueprint->GetAllGraphs(AllGraphs);

		for (UEdGraph* Graph : AllGraphs)
		{
			if (!IsValid(Graph) || Graph->HasAnyFlags(RF_Transient))
			{
				continue;
			}
			Graphs.Add(Graph);
		}
	}
	return Graphs;
}

UK2Node_PromotableOperator* CreateOpNode(const FName OpName, UEdGraph* Graph, const int32 AdditionalPins)
{
	if (!Graph)
	{
		return nullptr;
	}

	// The spawner will be null if type promo isn't enabled
	if (UBlueprintFunctionNodeSpawner* Spawner = FTypePromotion::GetOperatorSpawner(OpName))
	{
		// Spawn a new node!
		IBlueprintNodeBinder::FBindingSet Bindings;
		FVector2D SpawnLoc{};
		UK2Node_PromotableOperator* NewOpNode = Cast<UK2Node_PromotableOperator>(Spawner->Invoke(Graph, Bindings, SpawnLoc));
		check(NewOpNode);

		// Add the necessary number of additional pins
		for (int32 i = 0; i < AdditionalPins; ++i)
		{
			NewOpNode->AddInputPin();
		}

		return NewOpNode;
	}

	return nullptr;
}

void UBlueprintEditorLibrary::UpgradeOperatorNodes(UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return;
	}

	if (!TypePromoDebug::IsTypePromoEnabled())
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Type Promotion is not enabled! Cannot upgrade operator nodes. Set 'BP.TypePromo.IsEnabled' to true and try again.");
		return;
	}

	TArray<UEdGraph*> AllGraphs;
	Blueprint->GetAllGraphs(AllGraphs);
	Blueprint->Modify();

	/**
	* Used to help us restore the default values of any pins that may have changed their types
	* during replacement. 
	*/
	struct FRestoreDefaultsHelper
	{
		FEdGraphPinType PinType {};

		FString DefaultValue = TEXT("");

		TObjectPtr<UObject> DefaultObject = nullptr;

		FText DefaultTextValue = FText::GetEmpty();
	};

	TMap<FName, FRestoreDefaultsHelper> PinTypeMap;

	for (UEdGraph* Graph : AllGraphs)
	{
		check(Graph);

		Graph->Modify();

		for (int32 i = Graph->Nodes.Num() - 1; i >= 0; --i)
		{
			PinTypeMap.Reset();
			
			// Not every function that we want to upgrade is a CommunicativeBinaryOpNode
			// Some are just regular CallFunction nodes; Vector + Float is an example of this
			if (UK2Node_CallFunction* OldOpNode = Cast<UK2Node_CallFunction>(Graph->Nodes[i]))
			{
				UFunction* Func = OldOpNode->GetTargetFunction();
				UEdGraph* OwningGraph = OldOpNode->GetGraph();
				const bool bHadAnyConnections = InternalBlueprintEditorLibrary::NodeHasAnyConnections(OldOpNode);

				// We should only be modifying nodes within the graph that we want
				ensure(OwningGraph == Graph);
				
				// Don't bother with non-promotable functions or things that are already promotable operators
				if (!FTypePromotion::IsFunctionPromotionReady(Func) || OldOpNode->IsA<UK2Node_PromotableOperator>())
				{
					continue;
				}

				// Keep track of the types of anything with a default value so they can be restored
				for (UEdGraphPin* Pin : OldOpNode->Pins)
				{
					if (Pin->Direction == EGPD_Input && Pin->LinkedTo.IsEmpty())
					{
						FRestoreDefaultsHelper RestoreData;
						RestoreData.PinType = Pin->PinType;
						RestoreData.DefaultValue = Pin->DefaultValue;
						RestoreData.DefaultObject = Pin->DefaultObject;
						RestoreData.DefaultTextValue = Pin->DefaultTextValue;

						PinTypeMap.Add(Pin->GetFName(), RestoreData);
					}
				}

				FName OpName = FTypePromotion::GetOpNameFromFunction(Func);

				UK2Node_CommutativeAssociativeBinaryOperator* BinaryOpNode = Cast<UK2Node_CommutativeAssociativeBinaryOperator>(OldOpNode);

				// Spawn a new node!
				UK2Node_PromotableOperator* NewOpNode = CreateOpNode(
					OpName,
					OwningGraph,
					BinaryOpNode ? BinaryOpNode->GetNumberOfAdditionalInputs() : 0
				);

				// If there is a node that is a communicative op node but is not promotable
				// then the node will be null
				if (!NewOpNode)
				{
					UE_LOGF(LogBlueprintEditorLib, Warning, "Failed to spawn new operator node!");
					continue;
				}

				NewOpNode->NodePosX = OldOpNode->NodePosX;
				NewOpNode->NodePosY = OldOpNode->NodePosY;

				InternalBlueprintEditorLibrary::ReplaceOldNodeWithNew(OldOpNode, NewOpNode);

				for (const TPair<FName, FRestoreDefaultsHelper>& Pair : PinTypeMap)
				{
					const FRestoreDefaultsHelper& OldPinData = Pair.Value;

					if (UEdGraphPin* Pin = NewOpNode->FindPin(Pair.Key))
					{
						if (NewOpNode->CanConvertPinType(Pin))
						{
							NewOpNode->ConvertPinType(Pin, OldPinData.PinType);
							Pin->DefaultValue = OldPinData.DefaultValue;
							Pin->DefaultObject = OldPinData.DefaultObject;
							Pin->DefaultTextValue = OldPinData.DefaultTextValue;
						}
					}
				}

				// Reset the new node to be wild card if there were no connections to the original node.
				// This is necessary because replacing the old node will attempt to reconcile any 
				// default values on the node, which can result in incorrect pin types and a default
				// value that doesn't match. 
				if(!bHadAnyConnections)
				{
					NewOpNode->ResetNodeToWildcard();
				}
			}
		}
	}
}

bool UBlueprintEditorLibrary::CompileBlueprint(UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return false;
	}
	// Skip saving this to avoid possible tautologies when saving and allow the user to manually save
	EBlueprintCompileOptions Flags = EBlueprintCompileOptions::SkipSave;
	FKismetEditorUtilities::CompileBlueprint(Blueprint, Flags);
	return Blueprint->Status != BS_Error;
}

UEdGraph* UBlueprintEditorLibrary::AddFunctionGraph(UBlueprint* Blueprint, const FString& FuncName)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Failed to add function graph, ensure that blueprint is not null!");
		return nullptr;
	}

	// Validate that the given name is appropriate for a new function graph
	FName GraphName;

	if (FKismetNameValidator(Blueprint).IsValid(FuncName) == EValidatorResult::Ok)
	{
		GraphName = FName(*FuncName);
	}
	else
	{
		static const FString NewFunctionString = TEXT("NewFunction");
		GraphName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, !FuncName.IsEmpty() ? FuncName : NewFunctionString);
	}

	Blueprint->Modify();
	UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(
		Blueprint, 
		GraphName,
		UEdGraph::StaticClass(), 
		UEdGraphSchema_K2::StaticClass()
	);

	FBlueprintEditorUtils::AddFunctionGraph<UFunction>(Blueprint, NewGraph, /* bIsUserCreated = */ true, /* SignatureFromObject = */ nullptr);

	return NewGraph;
}

namespace UE::BlueprintEditorLibrary::Private
{
	// Returns true if any graph on the Blueprint contains an event node whose effective
	// name matches EventName. Covers both override-style events (matched via
	// EventReference.GetMemberName) and custom events (matched via CustomFunctionName).
	static bool HasEventNodeFor(UBlueprint* Blueprint, FName EventName)
	{
		TArray<UEdGraph*> AllGraphs;
		Blueprint->GetAllGraphs(AllGraphs);
		for (UEdGraph* Graph : AllGraphs)
		{
			if (!Graph)
			{
				continue;
			}
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				if (UK2Node_Event* EventNode = Cast<UK2Node_Event>(Node))
				{
					if (EventNode->CustomFunctionName == EventName)
					{
						return true;
					}
					if (EventNode->bOverrideFunction && EventNode->EventReference.GetMemberName() == EventName)
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	// Populates Result with one FBlueprintFunctionInfo per function visible on Blueprint
	// that matches the shape selector (event-shape vs function-shape). Walks local graphs,
	// the parent class chain, explicit interface implementations, and native interfaces in
	// the parent chain. Deduplicates by FName across all sources.
	static TArray<FBlueprintFunctionInfo> CollectFunctionInfos(UBlueprint* Blueprint, bool bWantEvents)
	{
		TArray<FBlueprintFunctionInfo> Result;
		if (!Blueprint || !Blueprint->ParentClass)
		{
			UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint or missing parent class!");
			return Result;
		}

		// Mirrors the population logic in SMyBlueprint::CollectAllActions for the
		// "Override Function" menu — see Editor/Kismet/Private/SMyBlueprint.cpp.
		FBlueprintEditorUtils::ConformImplementedInterfaces(Blueprint);

		TSet<FName> Seen;

		auto AddInherited = [&Result, &Seen, bWantEvents, Blueprint](UFunction* Function)
		{
			if (UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(Function) != bWantEvents)
			{
				return;
			}
			const FName FuncName = Function->GetFName();
			if (Seen.Contains(FuncName))
			{
				return;
			}
			Seen.Add(FuncName);

			FBlueprintFunctionInfo Info;
			Info.Name = FuncName;
			Info.Description = Function->GetToolTipText();
			Info.bIsImplemented = bWantEvents
				? HasEventNodeFor(Blueprint, FuncName)
				: (FindObject<UEdGraph>(Blueprint, *FuncName.ToString()) != nullptr);
			Result.Add(MoveTemp(Info));
		};

		// Locally-defined items first — these are always implemented by definition.
		if (bWantEvents)
		{
			TArray<UEdGraph*> AllGraphs;
			Blueprint->GetAllGraphs(AllGraphs);
			for (UEdGraph* Graph : AllGraphs)
			{
				if (!Graph)
				{
					continue;
				}
				for (UEdGraphNode* Node : Graph->Nodes)
				{
					if (UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(Node))
					{
						const FName Name = CustomEvent->CustomFunctionName;
						if (Name == NAME_None || Seen.Contains(Name))
						{
							continue;
						}
						Seen.Add(Name);

						FBlueprintFunctionInfo Info;
						Info.Name = Name;
						Info.Description = CustomEvent->GetTooltipText();
						Info.bIsImplemented = true;
						Result.Add(MoveTemp(Info));
					}
				}
			}
		}
		else
		{
			for (UEdGraph* Graph : Blueprint->FunctionGraphs)
			{
				if (!Graph)
				{
					continue;
				}
				const FName Name = Graph->GetFName();
				if (Seen.Contains(Name))
				{
					continue;
				}
				Seen.Add(Name);

				FBlueprintFunctionInfo Info;
				Info.Name = Name;
				// Local function graphs don't carry tooltip metadata directly; leave description empty.
				Info.bIsImplemented = true;
				Result.Add(MoveTemp(Info));
			}
		}

		UClass* const ParentClass = Blueprint->SkeletonGeneratedClass
			? Blueprint->SkeletonGeneratedClass->GetSuperClass()
			: *Blueprint->ParentClass;

		// Inherited from the parent class chain.
		for (TFieldIterator<UFunction> FuncIt(ParentClass, EFieldIteratorFlags::IncludeSuper); FuncIt; ++FuncIt)
		{
			UFunction* Function = *FuncIt;
			if (UEdGraphSchema_K2::CanKismetOverrideFunction(Function)
				&& !FObjectEditorUtils::IsFunctionHiddenFromClass(Function, ParentClass)
				&& Blueprint->AllowFunctionOverride(Function))
			{
				AddInherited(Function);
			}
		}

		// Inherited from interfaces explicitly implemented by this Blueprint.
		for (const FBPInterfaceDescription& InterfaceDesc : Blueprint->ImplementedInterfaces)
		{
			UClass* InterfaceClass = InterfaceDesc.Interface.Get();
			if (!InterfaceClass)
			{
				continue;
			}
			for (TFieldIterator<UFunction> FuncIt(InterfaceClass, EFieldIteratorFlags::IncludeSuper); FuncIt; ++FuncIt)
			{
				UFunction* Function = *FuncIt;
				if (Function->GetFName() == UEdGraphSchema_K2::FN_ExecuteUbergraphBase)
				{
					continue;
				}
				AddInherited(Function);
			}
		}

		// Inherited from interfaces implemented natively by classes in the parent chain.
		for (UClass* TempClass = Blueprint->ParentClass; TempClass; TempClass = TempClass->GetSuperClass())
		{
			for (const FImplementedInterface& Interface : TempClass->Interfaces)
			{
				if (!Interface.Class)
				{
					continue;
				}
				for (TFieldIterator<UFunction> FuncIt(Interface.Class, EFieldIteratorFlags::IncludeSuper); FuncIt; ++FuncIt)
				{
					UFunction* Function = *FuncIt;
					if (UEdGraphSchema_K2::CanKismetOverrideFunction(Function))
					{
						AddInherited(Function);
					}
				}
			}
		}

		return Result;
	}
}

TArray<FBlueprintFunctionInfo> UBlueprintEditorLibrary::ListFunctions(UBlueprint* Blueprint)
{
	return UE::BlueprintEditorLibrary::Private::CollectFunctionInfos(Blueprint, /* bWantEvents = */ false);
}

TArray<FBlueprintFunctionInfo> UBlueprintEditorLibrary::ListEvents(UBlueprint* Blueprint)
{
	return UE::BlueprintEditorLibrary::Private::CollectFunctionInfos(Blueprint, /* bWantEvents = */ true);
}

UEdGraph* UBlueprintEditorLibrary::AddFunctionOverride(UBlueprint* Blueprint, FName FunctionName)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return nullptr;
	}
	if (FunctionName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid function name!");
		return nullptr;
	}

	FBlueprintEditorUtils::ConformImplementedInterfaces(Blueprint);

	UFunction* OverrideFunc = nullptr;
	UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(Blueprint, FunctionName, &OverrideFunc);
	if (!OverrideFuncClass || !OverrideFunc)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Function '%ls' is not overridable on this Blueprint.", *FunctionName.ToString());
		return nullptr;
	}

	if (UEdGraph* ExistingGraph = FindObject<UEdGraph>(Blueprint, *FunctionName.ToString()))
	{
		return ExistingGraph;
	}

	// An override may already exist as an event node. Function-graph and event-node forms
	// are mutually exclusive for the same parent function.
	if (FBlueprintEditorUtils::FindOverrideForFunction(Blueprint, OverrideFuncClass, FunctionName))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"Function '%ls' is already overridden as an event node on this Blueprint. Remove the existing event node first.",
			*FunctionName.ToString());
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateOverrideFunctionGraph", "Create Override Function Graph"));
	Blueprint->Modify();

	UEdGraph* const NewGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, FunctionName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());

	// Passing the parent UClass as SignatureFromObject is what marks this as an override:
	// terminator nodes inherit the parent's signature and a CallParentFunction node is emitted.
	FBlueprintEditorUtils::AddFunctionGraph<UClass>(Blueprint, NewGraph, /*bIsUserCreated=*/false, OverrideFuncClass);
	NewGraph->Modify();
	return NewGraph;
}

UK2Node_Event* UBlueprintEditorLibrary::AddEventOverride(UBlueprint* Blueprint, FName EventName, FIntPoint Position)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return nullptr;
	}
	if (EventName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid event name!");
		return nullptr;
	}

	FBlueprintEditorUtils::ConformImplementedInterfaces(Blueprint);

	UFunction* OverrideFunc = nullptr;
	UClass* const OverrideFuncClass = FBlueprintEditorUtils::GetOverrideFunctionClass(Blueprint, EventName, &OverrideFunc);
	if (!OverrideFuncClass || !OverrideFunc)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Event '%ls' is not overridable on this Blueprint.", *EventName.ToString());
		return nullptr;
	}
	if (!UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(OverrideFunc))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"Function '%ls' is not event-shape and cannot be placed as an event node.", *EventName.ToString());
		return nullptr;
	}

	UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(Blueprint);
	if (!EventGraph)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Blueprint '%ls' has no event graph.", *Blueprint->GetFriendlyName());
		return nullptr;
	}

	if (UK2Node_Event* Existing = FBlueprintEditorUtils::FindOverrideForFunction(Blueprint, OverrideFuncClass, EventName))
	{
		return Existing;
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateEventOverride", "Implement Event"));
	Blueprint->Modify();

	return FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_Event>(
		EventGraph,
		FVector2D(Position.X, Position.Y),
		EK2NewNodeFlags::SelectNewNode,
		[EventName, OverrideFuncClass](UK2Node_Event* NewInstance)
		{
			NewInstance->EventReference.SetExternalMember(EventName, OverrideFuncClass);
			NewInstance->bOverrideFunction = true;
		}
	);
}

void UBlueprintEditorLibrary::RemoveFunctionGraph(UBlueprint* Blueprint, FName FuncName)
{
	if (!Blueprint)
	{
		return;
	}

	// Find the function graph of this name
	UEdGraph* FunctionGraph = nullptr;
	for (UEdGraph* Graph : Blueprint->FunctionGraphs)
	{
		if (Graph->GetFName() == FuncName)
		{
			FunctionGraph = Graph;
			break;
		}
	}

	// Remove the function graph if we can
	if (FunctionGraph && FunctionGraph->bAllowDeletion)
	{
		Blueprint->Modify();
		InternalBlueprintEditorLibrary::CloseOpenEditors(Blueprint);
		FBlueprintEditorUtils::RemoveGraph(Blueprint, FunctionGraph, EGraphRemoveFlags::MarkTransient);
	}
	else
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Failed to remove function '%ls' on blueprint '%ls'!", *FuncName.ToString(), *Blueprint->GetFriendlyName());
	}
}

void UBlueprintEditorLibrary::RemoveUnusedNodes(UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return;
	}

	TArray<UEdGraph*> AllGraphs;
	Blueprint->GetAllGraphs(AllGraphs);
	Blueprint->Modify();

	for (UEdGraph* Graph : AllGraphs)
	{
		// Skip non-editable graphs
		if (!Graph || FBlueprintEditorUtils::IsGraphReadOnly(Graph))
		{
			continue;
		}

		Graph->Modify();
		int32 NumNodesRemoved = 0;

		for (int32 i = Graph->Nodes.Num() - 1; i >= 0; --i)
		{
			UEdGraphNode* Node = Graph->Nodes[i];

			// We only want to delete user facing nodes because this is meant 
			// to be a BP refactoring/cleanup tool. Anim graph nodes can still 
			// be valid with no pin connections made to them
			if (Node->CanUserDeleteNode() && 
				!Node->IsA<UAnimGraphNode_Base>() && 
				!Node->IsA<UEdGraphNode_Comment>() &&
				!InternalBlueprintEditorLibrary::NodeHasAnyConnections(Node))
			{
				Node->BreakAllNodeLinks();
				Graph->RemoveNode(Node);
				++NumNodesRemoved;
			}
		}

		// Notify a change to the graph if nodes have been removed
		if (NumNodesRemoved > 0)
		{
			Graph->NotifyGraphChanged();
		}
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

void UBlueprintEditorLibrary::RemoveGraph(UBlueprint* Blueprint, UEdGraph* Graph)
{
	if (!Blueprint || !Graph)
	{
		return;
	}

	InternalBlueprintEditorLibrary::CloseOpenEditors(Blueprint);
	FBlueprintEditorUtils::RemoveGraph(Blueprint, Graph, EGraphRemoveFlags::MarkTransient);
}

void UBlueprintEditorLibrary::RenameGraph(UEdGraph* Graph, const FString& NewNameStr)
{
	if (!Graph)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid graph given, failed to rename!");
		return;
	}
	
	// Validate that the given name is appropriate for a new function graph
	UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
	if (!BP)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Failed to find blueprint for graph!");
		return;
	}

	FString ValidatedNewName;

	if (FKismetNameValidator(BP).IsValid(NewNameStr) == EValidatorResult::Ok)
	{
		ValidatedNewName = NewNameStr;
	}
	else
	{
		static const FString RenamedGraphString = TEXT("NewGraph");
		ValidatedNewName = FBlueprintEditorUtils::FindUniqueKismetName(BP, !NewNameStr.IsEmpty() ? NewNameStr : RenamedGraphString).ToString();
	}

	FBlueprintEditorUtils::RenameGraph(Graph, ValidatedNewName);
}

UBlueprint* UBlueprintEditorLibrary::GetBlueprintAsset(UObject* Object)
{
	if(Object == nullptr)
	{
		return nullptr;
	}

	if(UBlueprint* BP = Cast<UBlueprint>(Object))
	{
		return BP;
	}
	else if(UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(Object))
	{
		return Cast<UBlueprint>(Class->ClassGeneratedBy);
	}

	UPackage* Package = nullptr;
	if(Object->IsAsset() && Object->GetExternalPackage() == nullptr)
	{
		Package = Object->GetPackage();
	}
	else
	{
		Package = Cast<UPackage>(Object);
	}

	if(Package)
	{
		// search for a public root level blueprint:
		TArray<UObject*> RootObjects;
		GetObjectsWithOuter(Package, RootObjects, EGetObjectsFlags::None, RF_Transient, EInternalObjectFlags::Garbage);
		for(UObject* Obj : RootObjects)
		{
			if(!Obj->HasAnyFlags(RF_Public))
			{
				continue;
			}
			
			if(UBlueprint* InnerBP = Cast<UBlueprint>(Obj))
			{
				return InnerBP;
			}
			else if(UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(Obj))
			{
				return Cast<UBlueprint>(Class->ClassGeneratedBy);
			}
		}
	}
	return nullptr;
}

UBlueprint* UBlueprintEditorLibrary::GetBlueprintForClass(UClass* Class, bool& bDoesClassHaveBlueprint)
{
	bDoesClassHaveBlueprint = false;
	if(!Class)
	{
		return nullptr;
	}

	if(UBlueprint* Result = Cast<UBlueprint>(Class->ClassGeneratedBy))
	{
		bDoesClassHaveBlueprint = true;
		return Result;
	}
	return nullptr;
}

void UBlueprintEditorLibrary::RefreshOpenEditorsForBlueprint(const UBlueprint* BP)
{
	// Get any open blueprint editors for this asset and refresh them if they match the given blueprint
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
	for (TSharedRef<IBlueprintEditor>& Editor : BlueprintEditorModule.GetBlueprintEditors())
	{
		if (TSharedPtr<FBlueprintEditor> BPEditor = StaticCastSharedPtr<FBlueprintEditor>(Editor.ToSharedPtr()))
		{
			if (BPEditor->GetBlueprintObj() == BP)
			{
				BPEditor->RefreshEditors();
			}
		}
	}
}

void UBlueprintEditorLibrary::RefreshAllOpenBlueprintEditors()
{
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
	for (TSharedRef<IBlueprintEditor>& Editor : BlueprintEditorModule.GetBlueprintEditors())
	{
		Editor->RefreshEditors();
	}
}

UClass* UBlueprintEditorLibrary::GetBlueprintParentClass(const UBlueprint* Blueprint)
{
	return Blueprint ? Blueprint->ParentClass : nullptr;
}

void UBlueprintEditorLibrary::ReparentBlueprint(UBlueprint* Blueprint, UClass* NewParentClass)
{
	if (!Blueprint || !NewParentClass)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Failed to reparent blueprint!");
		return;
	}

	if (NewParentClass == Blueprint->ParentClass)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "'%ls' is already parented to class '%ls'!", *Blueprint->GetFriendlyName(), *NewParentClass->GetName());
		return;
	}

	// There could be possible data loss if reparenting outside the current class hierarchy
	if (!Blueprint->ParentClass || !NewParentClass->GetDefaultObject()->IsA(Blueprint->ParentClass))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "'%ls' class hierarchy is changing, there could be possible data loss!", *Blueprint->GetFriendlyName());
	}

	Blueprint->ParentClass = NewParentClass;

	if (Blueprint->SimpleConstructionScript != nullptr)
	{
		Blueprint->SimpleConstructionScript->ValidateSceneRootNodes();
	}

	FBlueprintEditorUtils::RefreshAllNodes(Blueprint);
	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);

	EBlueprintCompileOptions CompileOptions
	{
			EBlueprintCompileOptions::SkipSave
		|	EBlueprintCompileOptions::UseDeltaSerializationDuringReinstancing
		|	EBlueprintCompileOptions::SkipNewVariableDefaultsDetection
	};

	// If compilation is enabled during PIE/simulation, references to the CDO might be held by a script variable.
	// Thus, we set the flag to direct the compiler to allow those references to be replaced during reinstancing.
	if (GEditor && GEditor->PlayWorld != nullptr)
	{
		CompileOptions |= EBlueprintCompileOptions::IncludeCDOInReferenceReplacement;
	}

	FKismetEditorUtilities::CompileBlueprint(Blueprint, CompileOptions);
}

bool UBlueprintEditorLibrary::GatherUnusedVariables(const UBlueprint* Blueprint, TArray<FProperty*>& OutProperties)
{
	if (!Blueprint)
	{
		return false;
	}

	bool bHasAtLeastOneVariableToCheck = false;

	for (TFieldIterator<FProperty> PropertyIt(Blueprint->SkeletonGeneratedClass, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		// Don't show delegate properties, there is special handling for these
		const bool bDelegateProp = Property->IsA(FDelegateProperty::StaticClass()) || Property->IsA(FMulticastDelegateProperty::StaticClass());
		const bool bShouldShowProp = (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintVisible) && !bDelegateProp);

		if (bShouldShowProp)
		{
			bHasAtLeastOneVariableToCheck = true;
			FName VarName = Property->GetFName();

			const int32 VarInfoIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VarName);
			const bool bHasVarInfo = (VarInfoIndex != INDEX_NONE);

			const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property);
			bool bIsTimeline = ObjectProperty &&
				ObjectProperty->PropertyClass &&
				ObjectProperty->PropertyClass->IsChildOf(UTimelineComponent::StaticClass());
			if (!bIsTimeline && bHasVarInfo && !FBlueprintEditorUtils::IsVariableUsed(Blueprint, VarName))
			{
				OutProperties.Add(Property);
			}
		}
	}

	return bHasAtLeastOneVariableToCheck;
}

int32 UBlueprintEditorLibrary::RemoveUnusedVariables(UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return 0;
	}

	// Gather FProperties from this BP and see if we can remove any
	TArray<FProperty*> VariableProperties;
	UBlueprintEditorLibrary::GatherUnusedVariables(Blueprint, VariableProperties);
	
	// No variables can be removed from this blueprint
	if (VariableProperties.Num() == 0)
	{
		return 0;
	}

	// Get the variables by name so that we can bulk remove them and print them out to the log
	TArray<FName> VariableNames;
	FString PropertyList;
	VariableNames.Reserve(VariableProperties.Num());
	for (int32 Index = 0; Index < VariableProperties.Num(); ++Index)
	{
		VariableNames.Add(VariableProperties[Index]->GetFName());
		if (PropertyList.IsEmpty())
		{
			PropertyList = UEditorEngine::GetFriendlyName(VariableProperties[Index]);
		}
		else
		{
			PropertyList += FString::Printf(TEXT(", %s"), *UEditorEngine::GetFriendlyName(VariableProperties[Index]));
		}
	}

	const int32 NumRemovedVars = VariableNames.Num();
	// Remove the variables by name
	FBlueprintEditorUtils::BulkRemoveMemberVariables(Blueprint, VariableNames);

	UE_LOGF(LogBlueprintEditorLib, Log, "The following variable(s) were deleted successfully: %ls.", *PropertyList);
	return NumRemovedVars;
}

UClass* UBlueprintEditorLibrary::GeneratedClass(UBlueprint* BlueprintObj)
{
	if (BlueprintObj)
	{
		if(BlueprintObj->GeneratedClass == nullptr)
		{
			UE_LOGF(LogBlueprintEditorLib, Warning, "Blueprint %ls does not have a generated class - consider compiling it", *BlueprintObj->GetPathName());
			return nullptr;
		}
		return BlueprintObj->GeneratedClass->GetAuthoritativeClass();
	}
	return nullptr;
}

void UBlueprintEditorLibrary::ChangeMemberVariableType(UBlueprint* Blueprint, const FName& VariableName, const FEdGraphPinType& NewType)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return;
	}
	
	if (VariableName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid variable name!");
		return;
	}
	
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VariableName);
	if (VarIndex == INDEX_NONE)
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Could not find variable: {VariableName}", VariableName);
		return;
	}

	if(NewType == FEdGraphPinType())
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Cannot change variable {VariableName} to unspecfied type", VariableName);
		return;
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	FBPVariableDescription& Variable = Blueprint->NewVariables[VarIndex];
			
	// Update the variable type only if it is different
	if (Variable.VarType == NewType)
	{
		UE_LOGFMT(
			LogBlueprintEditorLib, Warning, 
			"Variable {VariableName} is already of type {VariableType} - no action performed", 
			VariableName, UBlueprintEditorLibrary::PinTypeToJsonSchema(Variable.VarType, Blueprint->GeneratedClass));
		return;
	}
	
	Blueprint->Modify();
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	if ((NewType.PinCategory == UEdGraphSchema_K2::PC_Object) ||
		(NewType.PinCategory == UEdGraphSchema_K2::PC_Interface))
	{
		// if it's a PC_Object, then it should have an associated UClass object
		if(NewType.PinSubCategoryObject.IsValid())
		{
			const UClass* ClassObject = Cast<UClass>(NewType.PinSubCategoryObject.Get());
			check(ClassObject != nullptr);

			if (ClassObject->IsChildOf(AActor::StaticClass()))
			{
				// NOTE: Right now the code that stops hard AActor references from being set 
				// in unsafe places is tied to this flag
				Variable.PropertyFlags |= CPF_DisableEditOnTemplate;
			}
			else 
			{
				// clear the disable-default-value flag that might have been present (if this was
				// an AActor variable before)
				Variable.PropertyFlags &= ~(CPF_DisableEditOnTemplate);
			}
		}
	}
	else 
	{
		// clear the disable-default-value flag that might have been present (if this was an AActor 
		// variable before)
		Variable.PropertyFlags &= ~(CPF_DisableEditOnTemplate);
	}

	const bool bBecameBoolean = Variable.VarType.PinCategory != UEdGraphSchema_K2::PC_Boolean && 
		NewType.PinCategory == UEdGraphSchema_K2::PC_Boolean;
	const bool bBecameNotBoolean = Variable.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean && 
		NewType.PinCategory != UEdGraphSchema_K2::PC_Boolean;
	if (bBecameBoolean || bBecameNotBoolean)
	{
		Variable.FriendlyName = FName::NameToDisplayString(Variable.VarName.ToString(), bBecameBoolean);
	}

	Variable.VarType = NewType;
	
	// Make sure that the variable is no longer tagged for replication, and warn the user if the variable is no
	// longer going to be replicated:
	if((Variable.VarType.IsSet() || Variable.VarType.IsMap()) &&
		(Variable.RepNotifyFunc != NAME_None || Variable.PropertyFlags & CPF_Net || Variable.PropertyFlags & CPF_RepNotify))
	{
		UE_LOGFMT(
			LogBlueprintEditorLib, Warning, 
			"Maps and sets cannot be replicated - {0} has had its replication settings cleared", 
			VariableName);

		Variable.PropertyFlags &= ~CPF_Net;
		Variable.PropertyFlags &= ~CPF_RepNotify;
		Variable.RepNotifyFunc = NAME_None;
		Variable.ReplicationCondition = COND_None;
	}
}

void UBlueprintEditorLibrary::SetBlueprintVariableExposeOnSpawn(UBlueprint* Blueprint, const FName& VariableName, bool bExposeOnSpawn)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return;
	}
	
	if (VariableName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid variable name!");
		return;
	}
	
	if(bExposeOnSpawn)
	{
		FBlueprintEditorUtils::SetBlueprintVariableMetaData(Blueprint, VariableName, NULL, FBlueprintMetadata::MD_ExposeOnSpawn, TEXT("true"));
	}
	else
	{
		FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(Blueprint, VariableName, NULL, FBlueprintMetadata::MD_ExposeOnSpawn);
	} 
}

UBlueprint* UBlueprintEditorLibrary::CreateBlueprintAssetWithParent(const FString& AssetPath, UClass* ParentClass)
{
	if(!ParentClass)
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Cannot create a blueprint asset with null parent class");
		return nullptr;
	}
	
	// do not allow inheritance of function library blueprints or native function libraries that already have functions
	// bIsValidFunctionLibrary provides a carve out for UEditorFunctionLibrary and similar sentinel types:
	const bool bIsFunctionLibrary = ParentClass->IsChildOf(UBlueprintFunctionLibrary::StaticClass());
	const bool bIsValidFunctionLibrary = bIsFunctionLibrary && (ParentClass->Children == nullptr && ParentClass->HasAnyClassFlags(CLASS_Native));
	if(bIsFunctionLibrary && !bIsValidFunctionLibrary)
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Cannot create a blueprint asset from a function library: {ClassPath}", ParentClass->GetPathName());
		return nullptr;
	}

	// Validate base blueprint logic - this enforces 'blueprintable/notblueprintable'
	if(!bIsValidFunctionLibrary && !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass))
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Not allowed to create blueprint for class: {ClassPath} - is it Blueprintable or IsBlueprintBase?", ParentClass->GetPathName());
		return nullptr;
	}

	// interface classes require special handling - reject them:
	if(ParentClass->HasAnyClassFlags(CLASS_Interface))
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Cannot create a blueprint asset from an interface: {ClassPath}", ParentClass->GetPathName());
		return nullptr;
	}

	const FString PackageName = UPackageTools::SanitizePackageName(AssetPath);
	UPackage* Existing = FindObject<UPackage>(nullptr, *PackageName);
	if(Existing)
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Cannot create a blueprint asset because an asset with this name already exists: {PackageName}", PackageName);
		return nullptr;
	}

	UPackage* Pkg = CreatePackage(*PackageName);
	if(!Pkg)
	{	
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Create Package Failed: {PackageName}", PackageName);
		return nullptr;
	}

	FName BPName = FPackageName::GetShortFName(PackageName);
	
	UClass* BlueprintClass = nullptr;
	UClass* BlueprintGeneratedClass = nullptr;
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.GetBlueprintTypesForClass(ParentClass, BlueprintClass, BlueprintGeneratedClass);
	
	UBlueprint* BP = FKismetEditorUtilities::CreateBlueprint(
		ParentClass, 
		Pkg, 
		BPName, 
		bIsFunctionLibrary ? BPTYPE_FunctionLibrary : BPTYPE_Normal, 
		BlueprintClass, 
		BlueprintGeneratedClass);
	ensure(BP); // FKismetEditorUtilities::CreateBlueprint does not return null, if it does we should clean up the UPackage - somehow

	Pkg->SetAssetAccessSpecifier(EAssetAccessSpecifier::Public);

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(BP);

	// Mark the package dirty...
	Pkg->MarkPackageDirty();
	return BP;
}

void UBlueprintEditorLibrary::ListGraphNames(const UBlueprint* Blueprint, TArray<FName>& OutGraphNames)
{
	if(!IsValid(Blueprint))
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Invalid blueprint has no graphs");
		return;
	}

	TArray<UEdGraph*> Graphs;
	Blueprint->GetAllGraphs(Graphs);
	for(UEdGraph* Graph : Graphs)
	{
		if(!IsValid(Graph) || Graph->HasAnyFlags(RF_Transient))
		{
			continue;
		}
		OutGraphNames.Add(Graph->GetFName());
	}
}

void UBlueprintEditorLibrary::ListMemberVariableNames(const UBlueprint* Blueprint, TArray<FString>& OutMemberVariableNames, bool bIncludeInheritedMembers)
{
	if (!IsValid(Blueprint))
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Invalid blueprint has no member variables");
		return;
	}

	// Add variables declared directly on this blueprint
	for (const FBPVariableDescription& VarDesc : Blueprint->NewVariables)
	{
		OutMemberVariableNames.Add(VarDesc.VarName.ToString());
	}

	if (bIncludeInheritedMembers)
	{
		InternalBlueprintEditorLibrary::ForEachInheritedVariable(Blueprint, [&OutMemberVariableNames](const UClass* Class, const FProperty* P, const FBPVariableDescription* D)
			{
				OutMemberVariableNames.Add(InternalBlueprintEditorLibrary::GetVariableFullName(Class, P, D));
				return true; // we want all variables
			});
	}
}

TOptional<FEdGraphPinType> UBlueprintEditorLibrary::GetMemberVariableType(const UBlueprint* Blueprint, const FString& MemberVariableName)
{
	if (!IsValid(Blueprint))
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Invalid blueprint has no member variables");
		return {};
	}
	
	// first, search for a member declared on this blueprint:
	for (const FBPVariableDescription& VarDesc : Blueprint->NewVariables)
	{
		if(VarDesc.VarName.ToString().Equals(MemberVariableName))
		{
			return VarDesc.VarType;
		}
	}
	
	// no match within the class, search the hierarchy - looking for a fully qualified match:
	TOptional<FEdGraphPinType> Result;
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	InternalBlueprintEditorLibrary::ForEachInheritedVariable(Blueprint, [&Result, &MemberVariableName, Schema](const UClass* Class, const FProperty* P, const FBPVariableDescription* D)
		{
			const FString MemberPath = InternalBlueprintEditorLibrary::GetVariableFullName(Class, P, D);
			if(MemberPath.Equals(MemberVariableName))
			{
				FEdGraphPinType PinType;
				if(InternalBlueprintEditorLibrary::GetVariableType(P, D, PinType))
				{
					Result = PinType;
					return false; // done, matched
				}
			}
			return true; // keep searching
		});
	if(Result.IsSet())
	{
		return Result;
	}
	
	for (const FBPVariableDescription& VarDesc : Blueprint->NewVariables)
	{
		if(VarDesc.VarName.ToString().Equals(MemberVariableName, ESearchCase::IgnoreCase))
		{
			return VarDesc.VarType;
		}
	}
	if(Result.IsSet())
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Case ignored to match member name - use a fully qualifed name to find inherited values unambiguously: {MemberName}", MemberVariableName);
		return Result;
	}

	// Still no match, do a short match in the hierarchy - this could result in ambiguous searches 
	// so leave a warning if we've matched:
	InternalBlueprintEditorLibrary::ForEachInheritedVariable(Blueprint, [&Result, &MemberVariableName, Schema](const UClass* Class, const FProperty* P, const FBPVariableDescription* D)
		{
			FString MemberName = InternalBlueprintEditorLibrary::GetVariableShortName(P, D);
			if(MemberName.Equals( MemberVariableName, ESearchCase::IgnoreCase))
			{
				FEdGraphPinType PinType;
				if(InternalBlueprintEditorLibrary::GetVariableType(P, D, PinType))
				{
					Result = PinType;
					return false; // done, matched
				}
			}
			return true; // keep searching
		});
	if(Result.IsSet())
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, "Short name matched inherited variable - consider fully qualifying the name: {MemberVariableName}", MemberVariableName);
		return Result;
	}

	// property not matched at all, log a warning:
	UE_LOGFMT(LogBlueprintEditorLib, Warning, "Could not find a match for: {MemberVariableName}", MemberVariableName);
	return {};
}

void UBlueprintEditorLibrary::SetBlueprintVariableExposeToCinematics(UBlueprint* Blueprint, const FName& VariableName, bool bExposeToCinematics)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return;
	}
	
	if (VariableName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid variable name!");
		return;
	}

	FBlueprintEditorUtils::SetInterpFlag(Blueprint, VariableName, bExposeToCinematics);
}

void UBlueprintEditorLibrary::SetBlueprintVariableInstanceEditable(UBlueprint* Blueprint, const FName& VariableName, bool bInstanceEditable)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return;
	}
	
	if (VariableName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid variable name!");
		return;
	}
	
	FBlueprintEditorUtils::SetBlueprintOnlyEditableFlag(Blueprint, VariableName, !bInstanceEditable);
}

EBlueprintVariableReplication UBlueprintEditorLibrary::GetBlueprintVariableReplication(
	UBlueprint* Blueprint, const FName& VariableName)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return EBlueprintVariableReplication::None;
	}

	if (VariableName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid variable name!");
		return EBlueprintVariableReplication::None;
	}

	const uint64* PropFlagPtr = FBlueprintEditorUtils::GetBlueprintVariablePropertyFlags(Blueprint, VariableName);
	if (!PropFlagPtr)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Variable '%ls' not found in Blueprint.", *VariableName.ToString());
		return EBlueprintVariableReplication::None;
	}

	if (!(*PropFlagPtr & CPF_Net))
	{
		return EBlueprintVariableReplication::None;
	}

	const FName RepNotifyFunc = FBlueprintEditorUtils::GetBlueprintVariableRepNotifyFunc(Blueprint, VariableName);
	return (RepNotifyFunc != NAME_None)
		? EBlueprintVariableReplication::RepNotify
		: EBlueprintVariableReplication::Replicated;
}

void UBlueprintEditorLibrary::SetBlueprintVariableReplication(UBlueprint* Blueprint, const FName& VariableName,
	EBlueprintVariableReplication Replication)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return;
	}

	if (VariableName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid variable name!");
		return;
	}

	uint64* PropFlagPtr = FBlueprintEditorUtils::GetBlueprintVariablePropertyFlags(Blueprint, VariableName);
	if (!PropFlagPtr)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Variable '%ls' not found in Blueprint.", *VariableName.ToString());
		return;
	}

	switch (Replication)
	{
	case EBlueprintVariableReplication::None:
		*PropFlagPtr &= ~CPF_Net;
		*PropFlagPtr &= ~CPF_RepNotify;
		FBlueprintEditorUtils::SetBlueprintVariableRepNotifyFunc(Blueprint, VariableName, NAME_None);
		{
			const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, VariableName);
			if (VarIndex != INDEX_NONE)
			{
				Blueprint->NewVariables[VarIndex].ReplicationCondition = COND_None;
			}
		}
		break;

	case EBlueprintVariableReplication::Replicated:
		*PropFlagPtr |= CPF_Net;
		*PropFlagPtr &= ~CPF_RepNotify;
		FBlueprintEditorUtils::SetBlueprintVariableRepNotifyFunc(Blueprint, VariableName, NAME_None);
		break;

	case EBlueprintVariableReplication::RepNotify:
		{
			*PropFlagPtr |= CPF_Net;
			const FString NewFuncName = FString::Printf(TEXT("OnRep_%s"), *VariableName.ToString());
			UEdGraph* FuncGraph = FindObject<UEdGraph>(Blueprint, *NewFuncName);
			if (!FuncGraph)
			{
				FuncGraph = FBlueprintEditorUtils::CreateNewGraph(
					Blueprint, FName(*NewFuncName), UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
				FBlueprintEditorUtils::AddFunctionGraph<UClass>(Blueprint, FuncGraph, false, nullptr);
			}
			if (FuncGraph)
			{
				FBlueprintEditorUtils::SetBlueprintVariableRepNotifyFunc(Blueprint, VariableName, FName(*NewFuncName));
				*PropFlagPtr |= CPF_RepNotify;
			}
		}
		break;
	}

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

FText UBlueprintEditorLibrary::GetBlueprintVariableCategory(UBlueprint* Blueprint, const FName& VariableName)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return FText::GetEmpty();
	}

	if (VariableName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid variable name!");
		return FText::GetEmpty();
	}

	return FBlueprintEditorUtils::GetBlueprintVariableCategory(Blueprint, VariableName, nullptr);
}

void UBlueprintEditorLibrary::SetBlueprintVariableCategory(UBlueprint* Blueprint, const FName& VariableName, const FText& NewCategory)
{
	if (!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid Blueprint!");
		return;
	}

	if (VariableName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Invalid variable name!");
		return;
	}

	FBlueprintEditorUtils::SetBlueprintVariableCategory(Blueprint, VariableName, nullptr, NewCategory);
}

FString UBlueprintEditorLibrary::GetCommentText(const UEdGraphNode_Comment* CommentNode)
{
	if (!CommentNode) { return FString(); }
	return CommentNode->NodeComment;
}

void UBlueprintEditorLibrary::SetCommentText(UEdGraphNode_Comment* CommentNode, const FString& NewText)
{
	if (!CommentNode) { return; }
	CommentNode->Modify();
	CommentNode->NodeComment = NewText;
	if (UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForNode(CommentNode))
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
	}
}

FLinearColor UBlueprintEditorLibrary::GetCommentColor(const UEdGraphNode_Comment* CommentNode)
{
	if (!CommentNode) { return FLinearColor::White; }
	return CommentNode->CommentColor;
}

void UBlueprintEditorLibrary::SetCommentColor(UEdGraphNode_Comment* CommentNode, FLinearColor Color)
{
	if (!CommentNode) { return; }
	CommentNode->Modify();
	CommentNode->CommentColor = Color;
	if (UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForNode(CommentNode))
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
	}
}

TArray<UK2Node*> UBlueprintEditorLibrary::GetNodesInComment(const UEdGraphNode_Comment* CommentNode)
{
	TArray<UK2Node*> Result;
	if (!CommentNode) { return Result; }
	for (UObject* Obj : CommentNode->GetNodesUnderComment())
	{
		if (UK2Node* K2Node = Cast<UK2Node>(Obj)) { Result.Add(K2Node); }
	}
	return Result;
}

FVector2D UBlueprintEditorLibrary::GetNodeSize(const UEdGraphNode* Node)
{
	if (!Node) { return FVector2D::ZeroVector; }
	const float W = Node->NodeWidth  > 0 ? Node->NodeWidth  : UEdGraphSchema_K2::EstimateNodeWidth(Node);
	const float H = Node->NodeHeight > 0 ? Node->NodeHeight : UEdGraphSchema_K2::EstimateNodeHeight(const_cast<UEdGraphNode*>(Node));
	return FVector2D(W, H);
}

bool UBlueprintEditorLibrary::Generic_AddMemberVariableWithValue(UBlueprint* Blueprint, FName MemberName, const uint8* DefaultValuePtr, const FProperty* DefaultValueProp)
{
	if(!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Attempted to add member variable to null blueprint");
		return false;	
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	FEdGraphPinType PinType;
	if( !Schema->ConvertPropertyToPinType(DefaultValueProp, PinType) )
	{
		return false;
	}

	FString ValueAsString;
	bool bGotDefaultValue = DefaultValueProp->ExportText_Direct(ValueAsString, DefaultValuePtr, DefaultValuePtr, nullptr, PPF_None);
	if(!bGotDefaultValue)
	{
		UE_LOGFMT(LogBlueprintEditorLib, Warning, 
			"Could not export the provided default value, variable {0} added to {1} will have incorrect default", 
			MemberName, Blueprint->GetPathName());
	}

	FName VarName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, MemberName.ToString(), Blueprint->SkeletonGeneratedClass);
	return FBlueprintEditorUtils::AddMemberVariable(Blueprint, VarName, PinType, ValueAsString);
}

DEFINE_FUNCTION(UBlueprintEditorLibrary::execAddMemberVariableWithValue)
{
	P_GET_OBJECT(UBlueprint, Blueprint);
	P_GET_PROPERTY(FNameProperty, MemberName);
	
	Stack.StepCompiledIn<FProperty>(nullptr);
	const FProperty* DefaultValueProp = Stack.MostRecentProperty;
	const uint8* DefaultValuePtr = Stack.MostRecentPropertyAddress;
	P_FINISH;

	if (!DefaultValueProp || !DefaultValuePtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("AddMemberVariable_MissingValue", "Failed to resolve default value and property type from AddMemberVariable.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	*(bool*)RESULT_PARAM = P_THIS->Generic_AddMemberVariableWithValue(Blueprint, MemberName, DefaultValuePtr, DefaultValueProp);
}

bool UBlueprintEditorLibrary::AddMemberVariable(UBlueprint* Blueprint, FName MemberName, const FEdGraphPinType& VariableType)
{
	if(!Blueprint)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Attempted to add member variable to null blueprint");
		return false;	
	}

	return FBlueprintEditorUtils::AddMemberVariable(
		Blueprint, 
		FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, MemberName.ToString(), Blueprint->SkeletonGeneratedClass),
		VariableType);
}

bool UBlueprintEditorLibrary::AddEventDispatcher(UBlueprint* Blueprint, FName Name)
{
	if (!::IsValid(Blueprint) || Name == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "AddEventDispatcher: invalid Blueprint or empty name");
		return false;
	}

	if (FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, Name) != INDEX_NONE)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"AddEventDispatcher: name '%ls' is already in use on '%ls'",
			*Name.ToString(), *Blueprint->GetPathName());
		return false;
	}

	Blueprint->Modify();

	FEdGraphPinType DelegateType;
	DelegateType.PinCategory = UEdGraphSchema_K2::PC_MCDelegate;
	if (!FBlueprintEditorUtils::AddMemberVariable(Blueprint, Name, DelegateType))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"AddEventDispatcher: AddMemberVariable failed for '%ls' on '%ls'",
			*Name.ToString(), *Blueprint->GetPathName());
		return false;
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	UEdGraph* SignatureGraph = FBlueprintEditorUtils::CreateNewGraph(Blueprint, Name, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	if (!SignatureGraph)
	{
		FBlueprintEditorUtils::RemoveMemberVariable(Blueprint, Name);
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"AddEventDispatcher: failed to create signature graph for '%ls'", *Name.ToString());
		return false;
	}

	SignatureGraph->bEditable = false;
	K2Schema->CreateDefaultNodesForGraph(*SignatureGraph);
	K2Schema->CreateFunctionGraphTerminators(*SignatureGraph, (UClass*)nullptr);
	K2Schema->AddExtraFunctionFlags(SignatureGraph, FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_Public);
	K2Schema->MarkFunctionEntryAsEditable(SignatureGraph, true);

	Blueprint->DelegateSignatureGraphs.Add(SignatureGraph);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	return true;
}

bool UBlueprintEditorLibrary::RemoveEventDispatcher(UBlueprint* Blueprint, FName Name)
{
	if (!::IsValid(Blueprint) || Name == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "RemoveEventDispatcher: invalid Blueprint or empty name");
		return false;
	}

	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, Name);
	if (VarIndex == INDEX_NONE)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"RemoveEventDispatcher: no dispatcher named '%ls' on '%ls'",
			*Name.ToString(), *Blueprint->GetPathName());
		return false;
	}

	if (Blueprint->NewVariables[VarIndex].VarType.PinCategory != UEdGraphSchema_K2::PC_MCDelegate)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"RemoveEventDispatcher: '%ls' is not an event dispatcher", *Name.ToString());
		return false;
	}

	Blueprint->Modify();

	// Remove the variable entry and any nodes referencing it in all graphs
	FBlueprintEditorUtils::RemoveMemberVariable(Blueprint, Name);

	// RemoveMemberVariable does not clean up the signature graph — do it explicitly
	UEdGraph* SignatureGraph = FBlueprintEditorUtils::GetDelegateSignatureGraphByName(Blueprint, Name);
	if (SignatureGraph)
	{
		Blueprint->DelegateSignatureGraphs.Remove(SignatureGraph);
		FBlueprintEditorUtils::RemoveGraph(Blueprint, SignatureGraph);
	}

	return true;
}

TArray<FName> UBlueprintEditorLibrary::ListEventDispatchers(const UBlueprint* Blueprint)
{
	TArray<FName> Result;
	if (!::IsValid(Blueprint))
	{
		return Result;
	}

	for (const FBPVariableDescription& Var : Blueprint->NewVariables)
	{
		if (Var.VarType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate)
		{
			Result.Add(Var.VarName);
		}
	}
	return Result;
}

bool UBlueprintEditorLibrary::AddEventDispatcherParameter(UBlueprint* Blueprint, FName DispatcherName,
	FName ParamName, const FEdGraphPinType& ParamType)
{
	if (!::IsValid(Blueprint) || DispatcherName == NAME_None || ParamName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "AddEventDispatcherParameter: invalid arguments");
		return false;
	}

	UEdGraph* SignatureGraph = FBlueprintEditorUtils::GetDelegateSignatureGraphByName(Blueprint, DispatcherName);
	if (!SignatureGraph)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"AddEventDispatcherParameter: no dispatcher named '%ls' on '%ls'",
			*DispatcherName.ToString(), *Blueprint->GetPathName());
		return false;
	}

	TArray<UK2Node_FunctionEntry*> EntryNodes;
	SignatureGraph->GetNodesOfClass(EntryNodes);
	if (EntryNodes.Num() == 0)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"AddEventDispatcherParameter: signature graph for '%ls' has no entry node",
			*DispatcherName.ToString());
		return false;
	}

	UK2Node_FunctionEntry* EntryNode = EntryNodes[0];
	if (EntryNode->FindPin(ParamName, EGPD_Output))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"AddEventDispatcherParameter: parameter '%ls' already exists on '%ls'",
			*ParamName.ToString(), *DispatcherName.ToString());
		return false;
	}

	EntryNode->Modify();
	EntryNode->CreateUserDefinedPin(ParamName, ParamType, EGPD_Output);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	return true;
}

bool UBlueprintEditorLibrary::RemoveEventDispatcherParameter(UBlueprint* Blueprint, FName DispatcherName,
	FName ParamName)
{
	if (!::IsValid(Blueprint) || DispatcherName == NAME_None || ParamName == NAME_None)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "RemoveEventDispatcherParameter: invalid arguments");
		return false;
	}

	UEdGraph* SignatureGraph = FBlueprintEditorUtils::GetDelegateSignatureGraphByName(Blueprint, DispatcherName);
	if (!SignatureGraph)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"RemoveEventDispatcherParameter: no dispatcher named '%ls' on '%ls'",
			*DispatcherName.ToString(), *Blueprint->GetPathName());
		return false;
	}

	TArray<UK2Node_FunctionEntry*> EntryNodes;
	SignatureGraph->GetNodesOfClass(EntryNodes);
	if (EntryNodes.Num() == 0)
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"RemoveEventDispatcherParameter: signature graph for '%ls' has no entry node",
			*DispatcherName.ToString());
		return false;
	}

	UK2Node_FunctionEntry* EntryNode = EntryNodes[0];
	if (!EntryNode->FindPin(ParamName, EGPD_Output))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning,
			"RemoveEventDispatcherParameter: no parameter '%ls' on dispatcher '%ls'",
			*ParamName.ToString(), *DispatcherName.ToString());
		return false;
	}

	EntryNode->Modify();
	EntryNode->RemoveUserDefinedPinByName(ParamName);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	return true;
}

FEdGraphPinType UBlueprintEditorLibrary::GetBasicTypeByName(FName TypeName)
{
	FEdGraphPinType Result;
	const TSet<FName> PrimitiveTypes = {
		UEdGraphSchema_K2::PC_Boolean, 
		UEdGraphSchema_K2::PC_Byte,
		UEdGraphSchema_K2::PC_Int,
		UEdGraphSchema_K2::PC_Int64,
		UEdGraphSchema_K2::PC_Real,
		UEdGraphSchema_K2::PC_Name,
		UEdGraphSchema_K2::PC_String,
		UEdGraphSchema_K2::PC_Text,
	};
	if(!PrimitiveTypes.Contains(TypeName))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Primitive type: %ls not recognized, defaulting to int", *TypeName.ToString());
		TypeName = UEdGraphSchema_K2::PC_Int;
	}
	Result.PinCategory = TypeName;
	if(TypeName == UEdGraphSchema_K2::PC_Real)
	{
		Result.PinSubCategory = UEdGraphSchema_K2::PC_Double;
	}
	return Result;
}
	
FEdGraphPinType UBlueprintEditorLibrary::GetStructType(const UScriptStruct* StructType)
{
	if(	StructType == nullptr || 
		!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(StructType))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Struct type: %ls not allowed, defaulting to int", StructType ? *StructType->GetPathName() : TEXT("null"));
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}
	
	FEdGraphPinType Result;
	Result.PinCategory = UEdGraphSchema_K2::PC_Struct;
	// the struct here is notionally const, and via PinSubCategoryObject should be extremely rare if they exist:
	Result.PinSubCategoryObject = const_cast<UScriptStruct*>(StructType);
	return Result;
}
	
FEdGraphPinType UBlueprintEditorLibrary::GetClassReferenceType(const UClass* ClassType)
{
	if(	ClassType == nullptr || 
		!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(ClassType))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Class type: %ls not allowed, defaulting to int", ClassType ? *ClassType->GetPathName() : TEXT("null"));
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}
	
	FEdGraphPinType Result;
	Result.PinCategory = UEdGraphSchema_K2::PC_Class;
	// the class here is notionally const, and via PinSubCategoryObject should be extremely rare if they exist:
	Result.PinSubCategoryObject = const_cast<UClass*>(ClassType);
	return Result;
}
	
FEdGraphPinType UBlueprintEditorLibrary::GetObjectReferenceType(const UClass* ObjectType)
{
	if(	ObjectType == nullptr || 
		!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(ObjectType))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Object reference type: %ls not allowed, defaulting to int", ObjectType ? *ObjectType->GetPathName() : TEXT("null"));
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}
	
	FEdGraphPinType Result;
	Result.PinCategory = UEdGraphSchema_K2::PC_Object;
	// the struct here is notionally const, and via PinSubCategoryObject should be extremely rare if they exist:
	Result.PinSubCategoryObject = const_cast<UClass*>(ObjectType);
	return Result;
}
	
FEdGraphPinType UBlueprintEditorLibrary::GetArrayType(const FEdGraphPinType& ContainedType)
{
	if(ContainedType.IsContainer())
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Containers cannot be nested directly, an intermediate struct type must be created. Defaulting to int");
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}

	FEdGraphPinType Result = ContainedType;
	Result.ContainerType = EPinContainerType::Array;
	return Result;
}
	
FEdGraphPinType UBlueprintEditorLibrary::GetSetType(const FEdGraphPinType& ContainedType)
{
	if(ContainedType.IsContainer())
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Containers cannot be nested directly, an intermediate struct type must be created. Defaulting to int");
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}

	if(!FBlueprintEditorUtils::HasGetTypeHash(ContainedType))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Key type must be hashable. Defaulting to int");
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}

	FEdGraphPinType Result= ContainedType;
	Result.ContainerType = EPinContainerType::Set;
	return Result;
}
	
FEdGraphPinType UBlueprintEditorLibrary::GetMapType(const FEdGraphPinType& KeyType,const FEdGraphPinType& ValueType)
{
	if(KeyType.IsContainer())
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Containers cannot be used as a key type, an intermediate struct type must be created. Defaulting to int");
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}

	if(ValueType.IsContainer())
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Containers cannot be as a value type, an intermediate struct type must be created. Defaulting to int");
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}

	if(!FBlueprintEditorUtils::HasGetTypeHash(KeyType))
	{
		UE_LOGF(LogBlueprintEditorLib, Warning, "Key type must be hashable. Defaulting to int");
		return GetBasicTypeByName(UEdGraphSchema_K2::PC_Int);
	}

	FEdGraphPinType Result = KeyType;
	Result.ContainerType = EPinContainerType::Map;
	Result.PinValueType = FEdGraphTerminalType::FromPinType(ValueType);
	return Result;
}

FString UBlueprintEditorLibrary::PinTypeToJsonSchema(const FEdGraphPinType& PinType, const UClass* SelfContext)
{
	if(PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
	{
		return TEXT("{\n    \"type\": \"exec\"\n}");
	}

	FCompilerResultsLog Discard;
	// we could use a scratch object or cache to reduce object graph pollution if needed,
	// but I want to reuse FPropertyToJsonSchemaObject rather than maintain a direct mapping
	UStruct* Scope = NewObject<UScriptStruct>();
	const FProperty* Property = FKismetCompilerUtilities::CreatePropertyOnScope(
		Scope, 
		TEXT("TestProperty"), 
		PinType, 
		const_cast<UClass*>(SelfContext),
		CPF_BlueprintVisible|CPF_BlueprintReadOnly, 
		GetDefault<UEdGraphSchema_K2>(), 
		Discard);
	if(!Property)
	{
		return TEXT("");
	}

	TSharedPtr<FJsonObject> PinSchema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Property);
	FString OutString;
	if(PinSchema)
	{
		TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutString);
		FJsonSerializer::Serialize(PinSchema, JsonWriter);
		JsonWriter->Close();	
	}
	return OutString;
}

namespace UE::Private
{
static void ForEachVisiblePin(const UK2Node* Node, TFunctionRef<bool(UEdGraphPin*)> Ftor)
{
	for(UEdGraphPin* Pin : Node->Pins)
	{
		if(Pin->bHidden)
		{
			continue;
		}

		if(!Ftor(Pin))
		{
			return;
		}
	}
}

static void ForEachVisibleInputPin(const UK2Node* Node, TFunctionRef<bool(UEdGraphPin*)> Ftor)
{
	ForEachVisiblePin(Node, [&Ftor](UEdGraphPin* Pin)
	{
		if(Pin->Direction == EGPD_Input)
		{
			return Ftor(Pin);
		}
		return true;
	});
}

static void ForEachVisibleOutputPin(const UK2Node* Node, TFunctionRef<bool(UEdGraphPin*)> Ftor)
{
	ForEachVisiblePin(Node, [&Ftor](UEdGraphPin* Pin)
	{
		if(Pin->Direction == EGPD_Output)
		{
			return Ftor(Pin);
		}
		return true;
	});
}

static UEdGraphPin* GetIfTypeMatches(const UClass* Class, UEdGraphPin* Pin, const FEdGraphPinType& Type)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	if(Type == FEdGraphPinType()) // unspecified default type, for this internal helper we treat that as a match
	{
		return Pin;
	}
	else if (K2Schema->ArePinTypesCompatible(Type, Pin->PinType, Class))
	{
		return Pin;
	}
	return nullptr;
}
}

TArray<FBlueprintGraphPin> UBlueprintEditorLibrary::ListAllPins(const UK2Node* Node, TEnumAsByte<EEdGraphPinDirection> InDirection)
{
	if(!::IsValid(Node))
	{
		return {};
	}

	TArray<FBlueprintGraphPin> Result;
	// first input pins - stable order will be more useful to callers
	if(InDirection == EGPD_MAX || InDirection == EGPD_Input)
	{
		UE::Private::ForEachVisibleInputPin(Node, [&Result](UEdGraphPin* Pin)
		{
			Result.Add(UBlueprintGraphPinLibrary::FromNativePin(Pin));
			return true;
		});
	}
	// then output pins
	if(InDirection == EGPD_MAX || InDirection == EGPD_Output)
	{
		UE::Private::ForEachVisibleOutputPin(Node, [&Result](UEdGraphPin* Pin)
		{
			Result.Add(UBlueprintGraphPinLibrary::FromNativePin(Pin));
			return true;
		});
	}
	return Result;
}

TArray<FBlueprintGraphPin> UBlueprintEditorLibrary::ListInputPins(const UK2Node* Node)
{
	return ListAllPins(Node, EGPD_Input);
}

TArray<FBlueprintGraphPin> UBlueprintEditorLibrary::ListOutputPins(const UK2Node* Node)
{
	return ListAllPins(Node, EGPD_Output);
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindInputPinByIndex(const UK2Node* Node, int32 Index, const FEdGraphPinType& Type)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	FBlueprintGraphPin Result;
	int32 Iter = 0;
	const UClass* Class = Node->GetBlueprintClassFromNode();
	UE::Private::ForEachVisibleInputPin(Node, [&Iter, Class, Node, Index, &Type, K2Schema, &Result](UEdGraphPin* Pin) {
		if(Iter != Index)
		{
			++Iter;
			return true; // wrong index, keep searching
		}

		Result = UBlueprintGraphPinLibrary::FromNativePin(UE::Private::GetIfTypeMatches(Class, Pin, Type));
		if(!::IsValid(Result.Node))
		{
			UE_LOGFMT(LogBlueprint, Warning,
				"Type on input pin {pin_index} named {pin_name} on {node} did not match {request_type}",
				Index,
				Pin->PinName,
				Node->GetPathName(),
				PinTypeToJsonSchema(Type, Class) );
		}
		return false; // index matched, all done
	});

	if(!::IsValid(Result.Node))
	{
		UE_LOGFMT(LogBlueprint, Warning, "Missing input pin at: {pin_index}, Pin count: {iter}, on node {node}", Index, Iter, Node->GetPathName());
	}
	return Result;
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindInputPin(const UK2Node* Node, FName PinName, const FEdGraphPinType& Type)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	FBlueprintGraphPin Result;
	const UClass* Class = Node->GetBlueprintClassFromNode();
	UE::Private::ForEachVisibleInputPin(Node, [PinName, Class, Node, &Type, &Result](UEdGraphPin* Pin)
	{
		if(Pin->PinName != PinName)
		{
			return true;
		}

		Result = UBlueprintGraphPinLibrary::FromNativePin(UE::Private::GetIfTypeMatches(Class, Pin, Type));
		if(!::IsValid(Result.Node))
		{
			UE_LOGFMT(LogBlueprint, Warning,
				"Type of input pin {pin_name} on {node} did not match {request_type}",
				Pin->PinName,
				Node->GetPathName(),
				PinTypeToJsonSchema(Type, Class) );
		}
		return false;
	});

	if(!::IsValid(Result.Node))
	{
		UE_LOGFMT(LogBlueprint, Warning, "Missing input pin named: {pin_name}, on node {node}", PinName, Node->GetPathName());
	}
	return Result;
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindOutputPinByIndex(const UK2Node* Node, int32 Index, const FEdGraphPinType& Type)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	FBlueprintGraphPin Result;
	int32 Iter = 0;
	const UClass* Class = Node->GetBlueprintClassFromNode();
	UE::Private::ForEachVisibleOutputPin(Node, [&Iter, Class, Node, Index, &Type, K2Schema, &Result](UEdGraphPin* Pin)
	{
		if(Iter != Index)
		{
			++Iter;
			return true; // wrong index, keep searching
		}

		Result = UBlueprintGraphPinLibrary::FromNativePin(UE::Private::GetIfTypeMatches(Class, Pin, Type));
		if(!::IsValid(Result.Node))
		{
			UE_LOGFMT(LogBlueprint, Warning,
				"Type on output pin {pin_index} named {pin_name} on {node} did not match {request_type}",
				Index,
				Pin->PinName,
				Node->GetPathName(),
				PinTypeToJsonSchema(Type, Class) );
		}
		return false; // index matched, all done
	});

	if(!::IsValid(Result.Node))
	{
		UE_LOGFMT(LogBlueprint, Warning, "Missing output pin at: {pin_index}, Pin count: {iter}, on node {node}", Index, Iter, Node->GetPathName());
	}
	return Result;
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindOutputPin(const UK2Node* Node, FName PinName, const FEdGraphPinType& Type)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	FBlueprintGraphPin Result;
	const UClass* Class = Node->GetBlueprintClassFromNode();
	UE::Private::ForEachVisibleOutputPin(Node, [Class, Node, PinName, &Type, &Result](UEdGraphPin* Pin)
	{
		if(Pin->PinName != PinName)
		{
			return true;
		}

		Result = UBlueprintGraphPinLibrary::FromNativePin(UE::Private::GetIfTypeMatches(Class, Pin, Type));
		if(!::IsValid(Result.Node))
		{
			UE_LOGFMT(LogBlueprint, Warning,
				"Type of output pin {pin_name} on {node} did not match {request_type}",
				Pin->PinName,
				Node->GetPathName(),
				PinTypeToJsonSchema(Type, Class) );
		}
		return false;
	});

	if(!::IsValid(Result.Node))
	{
		UE_LOGFMT(LogBlueprint, Warning, "Missing output pin named: {pin_name}, on node {node}", PinName, Node->GetPathName());
	}
	return Result;
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindExecutePin(const UK2Node* Node)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	FBlueprintGraphPin Result;
	UE::Private::ForEachVisibleInputPin(Node, [&Result](UEdGraphPin* Pin)
	{
		if(UEdGraphSchema_K2::IsExecPin(*Pin) && Pin->PinName == UEdGraphSchema_K2::PN_Execute)
		{
			Result = UBlueprintGraphPinLibrary::FromNativePin(Pin);
			return false;
		}
		return true;
	});
	if(!::IsValid(Result.Node))
	{
		UE_LOGFMT(LogBlueprint, Warning, "No execute pin found on node {node}", Node->GetPathName());
	}
	return Result;
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindThenPin(const UK2Node* Node)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	FBlueprintGraphPin Result;
	UE::Private::ForEachVisibleOutputPin(Node, [&Result](UEdGraphPin* Pin)
	{
		if(UEdGraphSchema_K2::IsExecPin(*Pin) && Pin->PinName == UEdGraphSchema_K2::PN_Then)
		{
			Result = UBlueprintGraphPinLibrary::FromNativePin(Pin);
			return false;
		}
		return true;
	});
	if(!::IsValid(Result.Node))
	{
		UE_LOGFMT(LogBlueprint, Warning, "No then pin found on node {node}", Node->GetPathName());
	}
	return Result;
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindSelfPin(const UK2Node* Node)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	FBlueprintGraphPin Result;
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	UE::Private::ForEachVisibleInputPin(Node, [&Result, K2Schema](UEdGraphPin* Pin)
	{
		if(K2Schema->IsSelfPin(*Pin))
		{
			Result = UBlueprintGraphPinLibrary::FromNativePin(Pin);
			return false;
		}
		return true;
	});
	if(!::IsValid(Result.Node))
	{
		UE_LOGFMT(LogBlueprint, Warning, "No self pin found on node {node}", Node->GetPathName());
	}
	return Result;
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindResultPin(const UK2Node* Node)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	UEdGraphPin* Result = nullptr;
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	bool bDidWarn = false;
	UE::Private::ForEachVisibleOutputPin(Node, [&Result, Node, &bDidWarn, K2Schema](UEdGraphPin* Pin)
	{
		if(UEdGraphSchema_K2::IsExecPin(*Pin))
		{
			return true;
		}
		if(Result)
		{
			Result = nullptr; // multiple outputs found, return empty
			UE_LOGFMT(LogBlueprint, Warning, "Multiple result pins found on node {node}", Node->GetPathName());
			bDidWarn = true;
			return false;
		}
		Result = Pin;
		return true;
	});
	if(!Result && !bDidWarn)
	{
		UE_LOGFMT(LogBlueprint, Warning, "No result pin found on node {node}", Node->GetPathName());
	}
	return UBlueprintGraphPinLibrary::FromNativePin(Result);
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindDataInputPin(const UK2Node* Node)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	UEdGraphPin* Result = nullptr;
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	bool bDidWarn = false;
	UE::Private::ForEachVisibleInputPin(Node, [&Result, &bDidWarn, Node, K2Schema](UEdGraphPin* Pin)
	{
		if(UEdGraphSchema_K2::IsExecPin(*Pin))
		{
			return true;
		}
		if(Result)
		{
			Result = nullptr; // multiple inputs found, return empty
			UE_LOGFMT(LogBlueprint, Warning, "Multiple input data pins found on node {node}", Node->GetPathName());
			bDidWarn = true;
			return false;
		}
		Result = Pin;
		return true;
	});
	if(!Result && !bDidWarn)
	{
		UE_LOGFMT(LogBlueprint, Warning, "No data input pin found on node {node}", Node->GetPathName());
	}
	return UBlueprintGraphPinLibrary::FromNativePin(Result);
}

void UBlueprintEditorLibrary::SetNodePos(UK2Node* Node, FIntPoint Pos)
{
	if(!::IsValid(Node))
	{
		return;
	}

	Node->Modify();

	Node->NodePosX = Pos.X;
	Node->NodePosY = Pos.Y;
}

FIntPoint UBlueprintEditorLibrary::GetNodePos(const UK2Node* Node)
{
	if(!::IsValid(Node))
	{
		return {0, 0};
	}

	return {Node->NodePosX, Node->NodePosY};
}

FString UBlueprintEditorLibrary::GetNodeTitle(const UK2Node* Node)
{
	if(!::IsValid(Node))
	{
		return TEXT("");
	}

	return Node->GetNodeTitle(ENodeTitleType::ListView).ToString();
}

FString UBlueprintEditorLibrary::GetNodeCategory(const UK2Node* Node)
{
	if (!::IsValid(Node))
	{
		return TEXT("");
	}

	return Node->GetMenuCategory().ToString();
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindElsePin(const UK2Node_IfThenElse* Node)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	return UBlueprintGraphPinLibrary::FromNativePin(Node->GetElsePin());
}

FBlueprintGraphPin UBlueprintEditorLibrary::FindConditionPin(const UK2Node_IfThenElse* Node)
{
	if(!::IsValid(Node))
	{
		return FBlueprintGraphPin{};
	}

	return UBlueprintGraphPinLibrary::FromNativePin(Node->GetConditionPin());
}

void UBlueprintEditorLibrary::SetCreateDelegateFunction(UK2Node_CreateDelegate* Node, FName FunctionName)
{
	if (!::IsValid(Node))
	{
		return;
	}
	Node->Modify();
	Node->SetFunction(FunctionName);
	if (UBlueprint* Blueprint = Node->GetBlueprint())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

FName UBlueprintEditorLibrary::GetCreateDelegateFunction(const UK2Node_CreateDelegate* Node)
{
	if (!::IsValid(Node))
	{
		return NAME_None;
	}
	return Node->GetFunctionName();
}

TArray<FName> UBlueprintEditorLibrary::ListCompatibleFunctionsForDelegate(const UK2Node_CreateDelegate* Node)
{
	TArray<FName> Result;
	if (!::IsValid(Node))
	{
		return Result;
	}

	const UFunction* Signature = Node->GetDelegateSignature();
	if (!Signature)
	{
		return Result;
	}

	UClass* ScopeClass = Node->GetScopeClass();
	if (!ScopeClass)
	{
		return Result;
	}

	for (TFieldIterator<UFunction> FuncIt(ScopeClass); FuncIt; ++FuncIt)
	{
		UFunction* Func = *FuncIt;
		if (Signature->IsSignatureCompatibleWith(Func)
			&& UEdGraphSchema_K2::FunctionCanBeUsedInDelegate(Func))
		{
			Result.Add(Func->GetFName());
		}
	}
	return Result;
}

#undef LOCTEXT_NAMESPACE	// "BlueprintEditorLibrary"
