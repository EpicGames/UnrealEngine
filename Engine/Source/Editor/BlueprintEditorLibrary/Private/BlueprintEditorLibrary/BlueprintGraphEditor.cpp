// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraphEditor.h"

#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "BlueprintEditorLibrary.h"
#include "BlueprintDelegateNodeSpawner.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintVariableNodeSpawner.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Kismet/KismetSystemLibrary.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SCS_Node.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_Event.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_Select.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_AddPinInterface.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_FormatText.h"
#include "K2Node_Select.h"
#include "K2Node_Switch.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Logging/StructuredLog.h"
#include "UObject/Class.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "BlueprintEditorLibrary/BlueprintGraphPin.h"
#include "BlueprintActionDatabase.h"

#define LOCTEXT_NAMESPACE "UBlueprintGraphEditor"

namespace
{
	// Picks the structural variant only when the node opts in via
	// NodeCausesStructuralBlueprintChange(); otherwise the lighter "modified"
	// flag matches Epic's UI-side convention for pin add/remove
	// (see SGraphNodeSwitchStatement::OnAddPin, FBlueprintEditor::OnRemoveExecutionPin).
	void MarkBlueprintForPinChange(UK2Node* Node, UBlueprint* Blueprint)
	{
		check(Node);
		check(Blueprint);
		if (Node->NodeCausesStructuralBlueprintChange())
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
		else
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

UBlueprintGraphEditor::UBlueprintGraphEditor()
{
}

UBlueprintGraphEditor* UBlueprintGraphEditor::CreateAndEditFunctionGraph(UBlueprint* InBlueprint, const FString& FuncName)
{
	if(!InBlueprint)
	{
		return nullptr;
	}

	UEdGraph* InGraph = UBlueprintEditorLibrary::AddFunctionGraph(InBlueprint, FuncName);
	if(!InGraph)
	{
		return nullptr;
	}

	UBlueprintGraphEditor* GraphEditor = NewObject<UBlueprintGraphEditor>();
	GraphEditor->Blueprint = InBlueprint;
	GraphEditor->Graph = InGraph;
	if(InGraph->Nodes.Num() > 0 )
	{
		UEdGraphNode* Node = InGraph->Nodes[0];
		if(Node)
		{
			GraphEditor->SpawnLocation = FVector2D(Node->NodePosX, Node->NodePosY);
		}
	}
	return GraphEditor;
}

UBlueprintGraphEditor* UBlueprintGraphEditor::GetGraphEditorByName(UBlueprint* InBlueprint, FName GraphName)
{
	const auto FindGraphByName = [](const TArray<UEdGraph*>& GraphList, FName Name)
	{
		return Algo::FindByPredicate(GraphList, [Name](UEdGraph* Graph)
		{
			return Graph && Graph->GetFName() == Name;
		} );
	};
	if(!InBlueprint)
	{
		return nullptr;
	}

	TArray<UEdGraph*> AllGraphs;
	InBlueprint->GetAllGraphs(AllGraphs);
	UEdGraph *const* FoundGraph = FindGraphByName(AllGraphs, GraphName);
	if(!FoundGraph)
	{
		UE_LOGFMT(LogBlueprint, Error, "Failed to find graph named {GraphName} for editing in {Blueprint}", GraphName, InBlueprint->GetPathName());
		return nullptr;
	}

	UBlueprintGraphEditor* GraphEditor = NewObject<UBlueprintGraphEditor>();
	GraphEditor->Blueprint = InBlueprint;
	GraphEditor->Graph = *FoundGraph;
	return GraphEditor;
}

UBlueprintGraphEditor* UBlueprintGraphEditor::GetGraphEditor(UEdGraph* InGraph)
{
	if(!InGraph)
	{
		UE_LOGFMT(LogBlueprint, Error, "Cannot make a graph editor for a null graph");
		return nullptr;
	}

	UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForGraph(InGraph);
	if(!BP)
	{
		UE_LOGFMT(LogBlueprint, Error, "Provided graph is not part of a blueprint: {GraphName}", InGraph->GetPathName());
	}
	
	UBlueprintGraphEditor* GraphEditor = NewObject<UBlueprintGraphEditor>();
	GraphEditor->Blueprint = BP;
	GraphEditor->Graph = InGraph;
	return GraphEditor;
}

void UBlueprintGraphEditor::ListAllNodes(TArray<UK2Node*>& OutNodes)
{
	if(!IsValid())
	{
		return;
	}

	Graph->GetNodesOfClass(OutNodes);
}
	
void UBlueprintGraphEditor::ListNodesOfClass(TArray<UK2Node*>& OutNodes, TSubclassOf<UK2Node> Class)
{
	if(!IsValid() || !Class)
	{
		return;
	}

	for(UEdGraphNode* Node : Graph->Nodes)
	{
		if(Node->IsA(Class))
		{
			OutNodes.Add(CastChecked<UK2Node>(Node));
		}
	}
}

void UBlueprintGraphEditor::ListNodesWithErrors(TArray<UK2Node*>& OutNodes)
{
	if(!IsValid())
	{
		return;
	}

	for(UEdGraphNode* Node : Graph->Nodes)
	{
		if(UK2Node* K2Node = Cast<UK2Node>(Node))
		{
			if (K2Node->bHasCompilerMessage && !Node->ErrorMsg.IsEmpty() && Node->ErrorType == EMessageSeverity::Error)
			{
				OutNodes.Add(K2Node);
			}
		}
	}
}

void UBlueprintGraphEditor::ListNodesWithWarnings(TArray<UK2Node*>& OutNodes)
{
	if(!IsValid())
	{
		return;
	}

	for(UEdGraphNode* Node : Graph->Nodes)
	{
		if(UK2Node* K2Node = Cast<UK2Node>(Node))
		{
			if (K2Node->bHasCompilerMessage && !Node->ErrorMsg.IsEmpty() && Node->ErrorType == EMessageSeverity::Warning)
			{
				OutNodes.Add(K2Node);
			}
		}
	}
}

void UBlueprintGraphEditor::ListNodesWithNotes(TArray<UK2Node*>& OutNodes)
{
	if(!IsValid())
	{
		return;
	}
	
	for(UEdGraphNode* Node : Graph->Nodes)
	{
		if(UK2Node* K2Node = Cast<UK2Node>(Node))
		{
			if (K2Node->bHasCompilerMessage && !Node->ErrorMsg.IsEmpty() && Node->ErrorType == EMessageSeverity::Info)
			{
				OutNodes.Add(K2Node);
			}
		}
	}
}

bool UBlueprintGraphEditor::RemoveGraphInputParameter(FName InputName)
{
	if(!IsValid())
	{
		return false;
	}

	UK2Node_FunctionEntry* Entry = GetFunctionEntryNode();
	if(!Entry)
	{
		return false;
	}
	Entry->Modify();
	const bool bHasPin = Entry->FindPin(InputName, EGPD_Output) != nullptr;
	if(bHasPin)
	{
		Entry->RemoveUserDefinedPinByName(InputName);
	}
	return bHasPin;
}

FBlueprintGraphPin UBlueprintGraphEditor::FindGraphEntryPin() const
{
	UK2Node_FunctionEntry* Entry = GetFunctionEntryNode();
	if(!Entry)
	{
		return FBlueprintGraphPin{};
	}
	return UBlueprintEditorLibrary::FindThenPin(Entry);
}

UK2Node_Event* UBlueprintGraphEditor::FindEventNode(FName EventName) const
{
	if(!IsValid())
	{
		return nullptr;
	}

	if (!FBlueprintEditorUtils::IsEventGraph(Graph))
	{
		UE_LOGFMT(LogBlueprint, Warning, "Non event graph {Graph} does not allow event editing", Graph->GetPathName());
		return nullptr;
	}

	TArray<UK2Node_Event*> EventNodes;
	Graph->GetNodesOfClass(EventNodes);

	UK2Node_Event* Result = nullptr;
	for(UK2Node_Event* EventNode : EventNodes)
	{
		if(EventNode->GetFunctionName() == EventName)
		{
			if(!Result)
			{
				Result = EventNode;
			}
			else
			{
				UE_LOGFMT(LogBlueprint, Warning, "Found ambiguous event node {EventName} for editing in {Graph}", EventName, Graph->GetPathName());
			}
		}
	}

	if(Result == nullptr)
	{
		UE_LOGFMT(LogBlueprint, Warning, "No event node {EventName} for in {Graph}", EventName, Graph->GetPathName());
	}

	return Result;
}

UK2Node_FunctionResult* UBlueprintGraphEditor::AddGraphOutputParameter(FName OutputName, const FEdGraphPinType& PinType)
{
	if(!IsValid())
	{
		return nullptr;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	TArray<UK2Node_FunctionResult*> ResultNodes;
	Graph->GetNodesOfClass(ResultNodes);
	if(ResultNodes.Num() == 0)
	{
		if(UK2Node_FunctionResult* Result = AddReturnNode())
		{
			ResultNodes.Add(Result);
		}
		else
		{
			UE_LOGFMT(LogBlueprint, Error, "Failed to create a result node in order to create an output parameter");
			return nullptr;
		}
	}

	for(UK2Node_FunctionResult* Result : ResultNodes)
	{
		Result->Modify();
		UEdGraphPin* Pin = Result->CreateUserDefinedPin(OutputName, 
			PinType != FEdGraphPinType() ? PinType : UBlueprintEditorLibrary::GetBasicTypeByName(UEdGraphSchema_K2::PC_Boolean), 
			EGPD_Input);
	}
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	return ResultNodes[0];
}


bool UBlueprintGraphEditor::RemoveGraphOutputParameter(FName OutputName)
{
	if(!IsValid())
	{
		return false;
	}

	TArray<UK2Node_FunctionResult*> TargetNodes;
	Graph->GetNodesOfClass(TargetNodes);

	bool bDoesAnyResultNodeHavePin = false;
	for (UK2Node_EditablePinBase* TargetNode : TargetNodes)
	{
		if(TargetNode->UserDefinedPinExists(OutputName))
		{
			bDoesAnyResultNodeHavePin = true;
			break;
		}
	}
	if(!bDoesAnyResultNodeHavePin)
	{
		return false;
	}

	for (UK2Node_EditablePinBase* TargetNode : TargetNodes)
	{
		TargetNode->Modify();
	}
	
	for (UK2Node_EditablePinBase* TargetNode : TargetNodes)
	{
		TargetNode->RemoveUserDefinedPinByName(OutputName);
	}
	
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	
	for (UK2Node_EditablePinBase* TargetNode : TargetNodes)
	{
		const bool bCurDisableOrphanSaving = TargetNode->bDisableOrphanPinSaving;
		TargetNode->bDisableOrphanPinSaving = true;
		TargetNode->ReconstructNode();
		TargetNode->bDisableOrphanPinSaving = bCurDisableOrphanSaving;

		K2Schema->HandleParameterDefaultValueChanged(TargetNode);
	}
	return true;
}

void UBlueprintGraphEditor::ListLocalVariableNames(TArray<FString>& OutLocalVariableNames) const
{
	if(!IsValid())
	{
		return;
	}

	if(!FBlueprintEditorUtils::DoesSupportLocalVariables(Graph))
	{
		UE_LOGFMT(LogBlueprint, Error, "Graph does not support locals {GraphName}", Graph->GetPathName());
		return;
	}

	TArray<UK2Node_FunctionEntry*> EntryNodes;
	Graph->GetNodesOfClass(EntryNodes);

	// There should always be an entry node in the function graph
	if(EntryNodes.Num() == 0)
	{
		UE_LOGFMT(LogBlueprint, Error, "Function graph is malformed - has no entry nodes {GraphName}", Graph->GetPathName());
		return;
	}
				
	UK2Node_FunctionEntry* FuncEntry = EntryNodes[0];
	for (const FBPVariableDescription& LocalVar : FuncEntry->LocalVariables)
	{
		OutLocalVariableNames.Add(LocalVar.VarName.ToString());
	}
}

TOptional<FEdGraphPinType> UBlueprintGraphEditor::GetLocalVariableType(const FString& LocalVariableName) const
{
	if(!IsValid())
	{
		return {};
	}

	if(!FBlueprintEditorUtils::DoesSupportLocalVariables(Graph))
	{
		UE_LOGFMT(LogBlueprint, Error, "Graph does not support locals {GraphName}", Graph->GetPathName());
		return {};
	}
	
	const FBPVariableDescription* VariableDescription = FBlueprintEditorUtils::FindLocalVariable(Blueprint, Graph, FName(*LocalVariableName));
	if(!VariableDescription)
	{
		UE_LOGFMT(LogBlueprint, Warning, "No local {LocalName} found in {GraphName}", LocalVariableName, Graph->GetPathName());
		return {};
	}

	return VariableDescription->VarType;
}

TOptional<FString> UBlueprintGraphEditor::GetLocalVariableDefaultValue(const FString& LocalVariableName) const
{
	if(!IsValid())
	{
		return {};
	}

	if(!FBlueprintEditorUtils::DoesSupportLocalVariables(Graph))
	{
		UE_LOGFMT(LogBlueprint, Error, "Graph does not support locals {GraphName}", Graph->GetPathName());
		return {};
	}
	
	const FBPVariableDescription* VariableDescription = FBlueprintEditorUtils::FindLocalVariable(Blueprint, Graph, FName(*LocalVariableName));
	if(!VariableDescription)
	{
		UE_LOGFMT(LogBlueprint, Warning, "No local {LocalName} found in {GraphName}", LocalVariableName, Graph->GetPathName());
		return {};
	}

	return VariableDescription->DefaultValue;
}

bool UBlueprintGraphEditor::SetLocalVariableDefaultValue(const FString& LocalVariableName, const FString& NewDefaultValue)
{
	if(!IsValid())
	{
		return false;
	}

	if(!FBlueprintEditorUtils::DoesSupportLocalVariables(Graph))
	{
		UE_LOGFMT(LogBlueprint, Error, "Graph does not support locals {GraphName}", Graph->GetPathName());
		return false;
	}
	
	FName PropertyAsName(*LocalVariableName);
	FBPVariableDescription* VariableDescription = FBlueprintEditorUtils::FindLocalVariable(Blueprint, Graph, PropertyAsName);
	if(!VariableDescription)
	{
		UE_LOGFMT(LogBlueprint, Warning, "No local {LocalName} found in {GraphName}", LocalVariableName, Graph->GetPathName());
		return false;
	}

	// we have to get the underlying property, which may require a skeleton compile:
	const FProperty* VariableProperty = FindLocalByName(PropertyAsName);
	if(!VariableProperty)
	{
		UE_LOGFMT(LogBlueprint, Error, "No backign property found for {LocalName} found in {GraphName} - possibly missing skeleton compile", LocalVariableName, Graph->GetPathName());
		return false;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	void* PropertyOnScope =  VariableProperty->AllocateAndInitializeValue();
	ON_SCOPE_EXIT
	{
		VariableProperty->DestroyAndFreeValue(PropertyOnScope);
	};
	constexpr EPropertyPortFlags ImportTextPortFlags = EPropertyPortFlags::PPF_SerializedAsImportText;
	const TCHAR* Result = VariableProperty->ImportText_Direct(*NewDefaultValue, PropertyOnScope, nullptr, ImportTextPortFlags, GLog);
	if(Result == nullptr)
	{
		return false;
	}

	// the value imported correctly, lets store it:
	VariableDescription->DefaultValue = NewDefaultValue;
	return true;
}

bool UBlueprintGraphEditor::AddLocalVariable(FName LocalName, const FEdGraphPinType& PinType, const FString& Value)
{
	if(!IsValid())
	{
		return false;
	}

	FName VarName = FBlueprintEditorUtils::FindUniqueKismetName(
		Blueprint, LocalName.ToString(), FindUField<UFunction>(Blueprint->SkeletonGeneratedClass, Graph->GetFName()));
	return FBlueprintEditorUtils::AddLocalVariable(
		Blueprint, 
		Graph, 
		VarName, 
		PinType != FEdGraphPinType() ? 
			PinType : UBlueprintEditorLibrary::GetBasicTypeByName(UEdGraphSchema_K2::PC_Boolean),
		Value);
}

bool UBlueprintGraphEditor::RemoveLocalVariable(FName LocalName)
{
	if(!IsValid())
	{
		return false;
	}

	UStruct* Function = FindUField<UFunction>(Blueprint->SkeletonGeneratedClass, Graph->GetFName());
	if(!Function)
	{
		return false;
	}
	if(FBlueprintEditorUtils::FindLocalVariable(Blueprint, 
		Function, LocalName) == nullptr)
	{
		return false;
	}
		
	FBlueprintEditorUtils::RemoveLocalVariable(Blueprint, Function, LocalName);
	return true;
}

bool UBlueprintGraphEditor::AddMemberVariable(FName MemberName, const FEdGraphPinType& PinType, const FString& Value)
{
	if(!IsValid())
	{
		return false;
	}

	FName VarName = FBlueprintEditorUtils::FindUniqueKismetName(
		Blueprint, MemberName.ToString(), FindUField<UFunction>(Blueprint->SkeletonGeneratedClass, Graph->GetFName()));
	return FBlueprintEditorUtils::AddMemberVariable(
		Blueprint, 
		VarName,
		PinType != FEdGraphPinType() ? 
			PinType : UBlueprintEditorLibrary::GetBasicTypeByName(UEdGraphSchema_K2::PC_Boolean),
		Value);
}

bool UBlueprintGraphEditor::RemoveMemberVariable(FName MemberName)
{
	if(!IsValid())
	{
		return false;
	}

	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, MemberName);
	if (VarIndex == INDEX_NONE)
	{
		return false;
	}

	FBlueprintEditorUtils::RemoveMemberVariable(Blueprint, MemberName);
	return true;
}

void UBlueprintGraphEditor::SetIsPureFunction(bool bIsPure)
{
	if(!IsValid())
	{
		return;
	}

	UFunction* Function = GetSkeletonFunction();
	if(!Function)
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	SetFunctionFlagImpl(Function, EntryNode, FUNC_BlueprintPure, bIsPure);
}

void UBlueprintGraphEditor::SetIsCallInEditorFunction(bool bCallInEditor)
{
	if(!IsValid())
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	if (EntryNode)
	{
		EntryNode->Modify();
		EntryNode->MetaData.bCallInEditor = bCallInEditor;
	}
}

void UBlueprintGraphEditor::SetFunctionIsPublic()
{
	if(!IsValid())
	{
		return;
	}

	UFunction* Function = GetSkeletonFunction();
	if(!Function)
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Private, false);
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Protected, false);
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Public, true);
}

void UBlueprintGraphEditor::SetFunctionIsProtected()
{
	if(!IsValid())
	{
		return;
	}

	UFunction* Function = GetSkeletonFunction();
	if(!Function)
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Private, false);
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Protected, true);
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Public, false);
}

void UBlueprintGraphEditor::SetFunctionIsPrivate()
{
	if(!IsValid())
	{
		return;
	}

	UFunction* Function = GetSkeletonFunction();
	if(!Function)
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Private, true);
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Protected, false);
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Public, false);
}

void UBlueprintGraphEditor::SetIsConstFunction(bool bIsConst)
{
	if(!IsValid())
	{
		return;
	}

	UFunction* Function = GetSkeletonFunction();
	if(!Function)
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Const, bIsConst);
}

void UBlueprintGraphEditor::SetIsExecFunction(bool bIsExec)
{
	if(!IsValid())
	{
		return;
	}

	UFunction* Function = GetSkeletonFunction();
	if(!Function)
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	SetFunctionFlagImpl(Function, EntryNode, FUNC_Exec, bIsExec);
}

void UBlueprintGraphEditor::SetIsThreadSafeFunction(bool bIsThreadSafe)
{
	if(!IsValid())
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	if (EntryNode)
	{
		EntryNode->Modify();
		EntryNode->MetaData.bThreadSafe = bIsThreadSafe;
	}
}

void UBlueprintGraphEditor::SetIsUnsafeDuringActorConstructionFunction(bool bIsUnsafeDuringActorConstruction)
{
	if(!IsValid())
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	if (EntryNode)
	{
		EntryNode->Modify();
		EntryNode->MetaData.bIsUnsafeDuringActorConstruction = bIsUnsafeDuringActorConstruction;
	}
}

void UBlueprintGraphEditor::SetIsDeprecatedFunction(bool bIsDeprecated)
{
	if(!IsValid())
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	if (EntryNode)
	{
		EntryNode->Modify();
		EntryNode->MetaData.bIsDeprecated = bIsDeprecated;
	}
}

void UBlueprintGraphEditor::SetDeprecationMessageOnFunction(FText Message)
{
	if(!IsValid())
	{
		return;
	}

	UK2Node_FunctionEntry* EntryNode = GetFunctionEntryNode();
	if (EntryNode)
	{
		EntryNode->Modify();
		EntryNode->MetaData.DeprecationMessage = Message.ToString();
	}
}


UK2Node_CallFunction* UBlueprintGraphEditor::AddCallFunctionNode(const FString& FunctionPath)
{
	if(!IsValid())
	{
		return nullptr;
	}

	TSoftObjectPtr<UFunction> FunctionSoftPtr = TSoftObjectPtr<UFunction>(FSoftObjectPath(FunctionPath));
	const UFunction* Function = FunctionSoftPtr.LoadSynchronous();
	if (!Function && Blueprint->SkeletonGeneratedClass)
	{
		// check for a local function:
		Function = Blueprint->SkeletonGeneratedClass->FindFunctionByName(*FunctionPath);
	}

	if(!Function)
	{
		return nullptr;
	}

	UBlueprintFunctionNodeSpawner* Spawner = UBlueprintFunctionNodeSpawner::Create(Function);
	UK2Node_CallFunction* Result = CastChecked<UK2Node_CallFunction>(Spawner->Invoke(Graph, {}, GetNodeLocation()));
	if(!Result)
	{
		return nullptr;
	}

	HandleNodeSpawned(Result);
	return Result;
}

UK2Node_MacroInstance* UBlueprintGraphEditor::AddMacroNode(const FString& MacroPath)
{
	if(!IsValid())
	{
		return nullptr;
	}

	TSoftObjectPtr<UEdGraph> GraphSoftPtr = TSoftObjectPtr<UEdGraph>(FSoftObjectPath(MacroPath));
	UEdGraph* MacroGraph = GraphSoftPtr.LoadSynchronous();
	if(!MacroGraph)
	{
		UE_LOGFMT(LogBlueprint, Error, "Failed to find macro graph named {MacroPath}", MacroPath);
		return nullptr;
	}

	UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(UK2Node_MacroInstance::StaticClass());
	check(NodeSpawner != nullptr);

	NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(
		[MacroGraph](UEdGraphNode* NewNode, bool bIsTemplateNode)
		{
			UK2Node_MacroInstance* MacroNode = CastChecked<UK2Node_MacroInstance>(NewNode);
			MacroNode->SetMacroGraph(MacroGraph);
		}
	);
	UK2Node_MacroInstance* Result = CastChecked<UK2Node_MacroInstance>(NodeSpawner->Invoke(Graph, {}, GetNodeLocation()));
	if(!Result)
	{
		return nullptr;
	}

	HandleNodeSpawned(Result);
	return Result;
}

UK2Node_IfThenElse* UBlueprintGraphEditor::AddBranchNode()
{
	if(!IsValid())
	{
		return nullptr;
	}

	UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(UK2Node_IfThenElse::StaticClass());
	UK2Node_IfThenElse* Result = CastChecked<UK2Node_IfThenElse>(NodeSpawner->Invoke(Graph, {}, GetNodeLocation()));
	if(!Result)
	{
		return nullptr;
	}

	HandleNodeSpawned(Result);
	return Result;
}

UK2Node_FunctionResult* UBlueprintGraphEditor::AddReturnNode()
{
	if(!IsValid())
	{
		return nullptr;
	}

	UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(UK2Node_FunctionResult::StaticClass());
	UK2Node_FunctionResult* Result = CastChecked<UK2Node_FunctionResult>(NodeSpawner->Invoke(Graph, {}, GetNodeLocation()));
	if(!Result)
	{
		return nullptr;
	}

	HandleNodeSpawned(Result);
	return Result;
}

UK2Node_VariableGet* UBlueprintGraphEditor::AddGetMemberVariableNode(FName MemberName, const FString& ClassPath)
{
	if(!IsValid())
	{
		return nullptr;
	}

	const UClass* Scope = Blueprint->SkeletonGeneratedClass;
	if(!ClassPath.IsEmpty())
	{
		Scope = GetClassFromPath(ClassPath);

		if(!Scope)
		{
			UE_LOGFMT(LogBlueprint, Error, "Failed to find class {ClassPath}", ClassPath);
			return nullptr;
		}
	}

	if(!Scope)
	{
		UE_LOGFMT(LogBlueprint, Error, "Failed to resolve scope");
		return nullptr;
	}

	const FProperty* Property = Scope->FindPropertyByName(MemberName);
	if(!Property)
	{
		UE_LOGFMT(LogBlueprint, Error, "Failed to find member named {MemberName}", MemberName);
		return nullptr;
	}
	
	UBlueprintVariableNodeSpawner* Spawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableGet::StaticClass(), Property);
	UK2Node_VariableGet* Result = CastChecked<UK2Node_VariableGet>(Spawner->Invoke(Graph, {}, GetNodeLocation()));
	if(!Result)
	{
		return nullptr;
	}

	HandleNodeSpawned(Result);
	return Result;
}

UK2Node_VariableSet* UBlueprintGraphEditor::AddSetMemberVariableNode(FName MemberName, const FString& ClassPath)
{
	if(!IsValid())
	{
		return nullptr;
	}

	const UClass* Scope = Blueprint->SkeletonGeneratedClass;
	if(!ClassPath.IsEmpty())
	{
		Scope = GetClassFromPath(ClassPath);
	}

	if(!::IsValid(Scope))
	{
		UE_LOGFMT(LogBlueprint, Error, "Failed to find Class for AddSetMemberVariableNode: {ClassPath}", ClassPath);
		return nullptr;
	}

	const FProperty* Property = Scope->FindPropertyByName(MemberName);
	if(!Property)
	{
		UE_LOGFMT(LogBlueprint, Error, "Failed to find member for AddSetMemberVariableNode: {MemberName}", MemberName);
		return nullptr;
	}
	
	UBlueprintVariableNodeSpawner* Spawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableSet::StaticClass(), Property);
	UK2Node_VariableSet* Result = CastChecked<UK2Node_VariableSet>(Spawner->Invoke(Graph, {}, GetNodeLocation()));
	if(!Result)
	{
		return nullptr;
	}

	HandleNodeSpawned(Result);
	return Result;
}

UK2Node_VariableGet* UBlueprintGraphEditor::AddGetLocalVariableNode(FName LocalName)
{
	if(!IsValid())
	{
		return nullptr;
	}

	UBlueprintVariableNodeSpawner* Spawner = nullptr;
	FBPVariableDescription* VariableDescription = FBlueprintEditorUtils::FindLocalVariable(Blueprint, Graph, LocalName);
	if(VariableDescription)
	{
		// create from local
		FMemberReference Reference;
		Reference.SetLocalMember(LocalName, Graph->GetName(), VariableDescription->VarGuid);
		Spawner = UBlueprintVariableNodeSpawner::CreateFromLocal(
			UK2Node_VariableGet::StaticClass(), Graph, *VariableDescription, Reference.ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass));	
	}
	else if(const FProperty* Input = FindParameterByName(LocalName))
	{
		// create from a parameter:
		Spawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableGet::StaticClass(), Input, Graph);
	}

	if(!Spawner)
	{
		return nullptr;
	}
	
	UK2Node_VariableGet* Result = CastChecked<UK2Node_VariableGet>(Spawner->Invoke(Graph, {}, GetNodeLocation()));
	if(!Result)
	{
		return nullptr;
	}

	HandleNodeSpawned(Result);
	return Result;
}

UK2Node_VariableSet* UBlueprintGraphEditor::AddSetLocalVariableNode(FName LocalName)
{
	if(!IsValid())
	{
		return nullptr;
	}

	FBPVariableDescription* VariableDescription = FBlueprintEditorUtils::FindLocalVariable(Blueprint, Graph, LocalName);
	if(!VariableDescription)
	{
		return nullptr;
	}
	
	FMemberReference Reference;
	Reference.SetLocalMember(LocalName, Graph->GetName(), VariableDescription->VarGuid);
	UBlueprintVariableNodeSpawner* Spawner = UBlueprintVariableNodeSpawner::CreateFromLocal(
		UK2Node_VariableSet::StaticClass(), Graph, *VariableDescription, Reference.ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass));
	UK2Node_VariableSet* Result = CastChecked<UK2Node_VariableSet>(Spawner->Invoke(Graph, {}, GetNodeLocation()));
	if(!Result)
	{
		return nullptr;
	}

	HandleNodeSpawned(Result);
	return Result;
}

UK2Node_CustomEvent* UBlueprintGraphEditor::AddCustomEventNode(const FString& EventName)
{
	if(!IsValid())
	{
		return nullptr;
	}

	if (!FBlueprintEditorUtils::IsEventGraph(Graph))
	{
		UE_LOGFMT(LogBlueprint, Warning, "Cannot add custom event to {Graph} - it is not an event graph", Graph->GetPathName());
		return nullptr;
	}

	UK2Node_CustomEvent* NewNode = NewObject<UK2Node_CustomEvent>(Graph);
	check(NewNode != nullptr);
	NewNode->CreateNewGuid();

	// position the node before invoking PostSpawnDelegate (in case it 
	// wishes to modify this positioning)
	const FVector2D Location = GetNodeLocation();
	NewNode->NodePosX = static_cast<int32>(Location.X);
	NewNode->NodePosY = static_cast<int32>(Location.Y);

	NewNode->CustomFunctionName =  FBlueprintEditorUtils::FindUniqueCustomEventName(Blueprint);
	NewNode->bIsEditable = true;
	NewNode->SetFlags(RF_Transactional);
	NewNode->AllocateDefaultPins();
	NewNode->PostPlacedNewNode();

	Graph->Modify();
	// the FBlueprintMenuActionItem should do the selecting
	Graph->AddNode(NewNode, /*bFromUI =*/true, /*bSelectNewNode =*/false);

	// Unfortunately we can't validate the name easilly without first spawning the node, but we can honor
	// name validity nonetheless:
	TSharedPtr<INameValidatorInterface> NameValidator = NewNode->MakeNameValidator();
	EValidatorResult ValidatorResult = NameValidator->IsValid(EventName);
	if(ValidatorResult != EValidatorResult::Ok)
	{
		UE_LOGFMT(LogBlueprint, Warning, "User provided name was invalid {ErrorMessage} - node named {FallbackName}", 
			INameValidatorInterface::GetErrorText(EventName, ValidatorResult).ToString(), NewNode->CustomFunctionName);
	}
	else
	{
		NewNode->CustomFunctionName = FName(*EventName);
	}

	HandleNodeSpawned(NewNode);

	return NewNode;
}

UK2Node_Event* UBlueprintGraphEditor::AddDispatcherEventNode(FName DispatcherName, UClass* DeclaringClass)
{
	if (!IsValid())
	{
		return nullptr;
	}

	if (!FBlueprintEditorUtils::IsEventGraph(Graph))
	{
		UE_LOGFMT(LogBlueprint, Warning,
			"Cannot add dispatcher event node to {Graph} - it is not an event graph",
			Graph->GetPathName());
		return nullptr;
	}

	if (!Blueprint->SkeletonGeneratedClass)
	{
		UE_LOGFMT(LogBlueprint, Warning,
			"Cannot add dispatcher event node - {Blueprint} has no skeleton class",
			Blueprint->GetName());
		return nullptr;
	}

	FMulticastDelegateProperty* DelegateProperty = FindFProperty<FMulticastDelegateProperty>(
		Blueprint->SkeletonGeneratedClass, DispatcherName);
	if (!DelegateProperty || !DelegateProperty->SignatureFunction)
	{
		UE_LOGFMT(LogBlueprint, Warning,
			"AddDispatcherEventNode: no dispatcher '{Name}' found on skeleton class",
			DispatcherName);
		return nullptr;
	}
	if (DeclaringClass)
	{
		UClass* EffectiveDeclaringClass = FBlueprintEditorUtils::GetSkeletonClass(DeclaringClass);
		if (!EffectiveDeclaringClass)
		{
			EffectiveDeclaringClass = DeclaringClass;
		}
		if (DelegateProperty->GetOwnerClass() != EffectiveDeclaringClass)
		{
			UE_LOGFMT(LogBlueprint, Warning,
				"AddDispatcherEventNode: dispatcher '{Name}' is not declared on '{Class}' (declared on '{Owner}')",
				DispatcherName, *EffectiveDeclaringClass->GetName(), *DelegateProperty->GetOwnerClass()->GetName());
			return nullptr;
		}
	}
	const UFunction* SignatureFunction = DelegateProperty->SignatureFunction;

	// The event name must be unique and must not match the dispatcher name — having both an event
	// stub function and a delegate variable with the same name causes a compile conflict.
	FName EventName = FBlueprintEditorUtils::FindUniqueKismetName(Blueprint, DispatcherName.ToString());
	UK2Node_CustomEvent* NewNode = UK2Node_CustomEvent::CreateFromFunction(
		GetNodeLocation(), Graph, EventName.ToString(), SignatureFunction);
	if (!NewNode)
	{
		return nullptr;
	}

	HandleNodeSpawned(NewNode);
	return NewNode;
}

bool UBlueprintGraphEditor::RetargetNodeClass(UEdGraphNode* Node, UClass* OldClass, UClass* NewClass)
{
	if (!Node || !OldClass || !NewClass)
	{
		return false;
	}

	if (UK2Node_DynamicCast* CastNode = Cast<UK2Node_DynamicCast>(Node))
	{
		const UClass* CurrentClass = CastNode->TargetType;
		if (CurrentClass == NewClass)
		{
			return true;
		}
		if (CurrentClass != OldClass)
		{
			return false;
		}
		CastNode->TargetType = NewClass;
		CastNode->ReconstructNode();
		return true;
	}

	if (UK2Node_CallFunction* CallNode = Cast<UK2Node_CallFunction>(Node))
	{
		const UClass* CurrentClass = CallNode->FunctionReference.GetMemberParentClass();
		if (CurrentClass == NewClass)
		{
			return true;
		}
		if (CurrentClass != OldClass)
		{
			return false;
		}
		CallNode->FunctionReference.SetExternalMember(
			CallNode->FunctionReference.GetMemberName(), NewClass);
		CallNode->ReconstructNode();
		return true;
	}

	if (UK2Node_Event* EventNode = Cast<UK2Node_Event>(Node))
	{
		const UClass* CurrentClass = EventNode->EventReference.GetMemberParentClass();
		if (CurrentClass == NewClass)
		{
			return true;
		}
		if (CurrentClass != OldClass)
		{
			return false;
		}
		EventNode->EventReference.SetExternalMember(
			EventNode->EventReference.GetMemberName(), NewClass);
		EventNode->ReconstructNode();
		return true;
	}

	if (UK2Node_BaseMCDelegate* DelegateNode = Cast<UK2Node_BaseMCDelegate>(Node))
	{
		const UClass* CurrentClass = DelegateNode->DelegateReference.GetMemberParentClass();
		if (CurrentClass == NewClass)
		{
			return true;
		}
		if (CurrentClass != OldClass)
		{
			return false;
		}
		DelegateNode->DelegateReference.SetExternalMember(
			DelegateNode->DelegateReference.GetMemberName(), NewClass);
		DelegateNode->ReconstructNode();
		return true;
	}

	return false;
}

TArray<FName> UBlueprintGraphEditor::ListComponentEvents(UActorComponent* Component) const
{
	TArray<FName> Result;
	if (!Component)
	{
		return Result;
	}
	for (TFieldIterator<FMulticastDelegateProperty> PropIt(Component->GetClass()); PropIt; ++PropIt)
	{
		if (PropIt->HasAnyPropertyFlags(CPF_BlueprintAssignable))
		{
			Result.Add(PropIt->GetFName());
		}
	}
	return Result;
}

UK2Node_ComponentBoundEvent* UBlueprintGraphEditor::AddComponentBoundEventNode(
	UActorComponent* Component, FName EventName)
{
	if (!IsValid() || !Component || EventName == NAME_None)
	{
		return nullptr;
	}
	if (!FBlueprintEditorUtils::IsEventGraph(Graph))
	{
		UE_LOGFMT(LogBlueprint, Warning, "Cannot add component bound event to {Graph} - it is not an event graph", Graph->GetPathName());
		return nullptr;
	}

	if (!Blueprint->SkeletonGeneratedClass)
	{
		UE_LOGFMT(LogBlueprint, Warning, "Cannot add component bound event - Blueprint {Blueprint} has not been compiled", Blueprint->GetName());
		return nullptr;
	}

	// Walk the SCS to find the FObjectProperty corresponding to this component template
	FObjectProperty* ComponentProperty = nullptr;
	if (Blueprint->SimpleConstructionScript)
	{
		for (USCS_Node* SCSNode : Blueprint->SimpleConstructionScript->GetAllNodes())
		{
			if (SCSNode->ComponentTemplate == Component)
			{
				ComponentProperty = FindFProperty<FObjectProperty>(Blueprint->SkeletonGeneratedClass, SCSNode->GetVariableName());
				break;
			}
		}
	}
	if (!ComponentProperty)
	{
		UE_LOGFMT(LogBlueprint, Warning, "Could not find a Blueprint property for component {Component}", Component->GetName());
		return nullptr;
	}

	FMulticastDelegateProperty* DelegateProperty = FindFProperty<FMulticastDelegateProperty>(Component->GetClass(), EventName);
	if (!DelegateProperty)
	{
		UE_LOGFMT(LogBlueprint, Warning, "Could not find delegate {EventName} on {ComponentClass}", EventName, Component->GetClass()->GetName());
		return nullptr;
	}

	// Return the existing node rather than creating a duplicate
	if (const UK2Node_ComponentBoundEvent* Existing = FKismetEditorUtilities::FindBoundEventForComponent(Blueprint, EventName, ComponentProperty->GetFName()))
	{
		return const_cast<UK2Node_ComponentBoundEvent*>(Existing);
	}

	UK2Node_ComponentBoundEvent* NewNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_ComponentBoundEvent>(
		Graph, GetNodeLocation(), EK2NewNodeFlags::SelectNewNode,
		[ComponentProperty, DelegateProperty](UK2Node_ComponentBoundEvent* Instance)
		{
			Instance->InitializeComponentBoundEventParams(ComponentProperty, DelegateProperty);
		}
	);

	if (NewNode)
	{
		HandleNodeSpawned(NewNode);
	}
	return NewNode;
}

UEdGraphNode_Comment* UBlueprintGraphEditor::AddCommentNode(const FString& CommentText, FVector2D Location, FVector2D Size)
{
	if (!IsValid())
	{
		return nullptr;
	}

	UEdGraphNode_Comment* CommentNode = NewObject<UEdGraphNode_Comment>(Graph);
	CommentNode->SetFlags(RF_Transactional);
	CommentNode->NodePosX = static_cast<int32>(Location.X);
	CommentNode->NodePosY = static_cast<int32>(Location.Y);
	CommentNode->NodeWidth = static_cast<int32>(Size.X);
	CommentNode->NodeHeight = static_cast<int32>(Size.Y);

	CommentNode->PostPlacedNewNode();
	CommentNode->AllocateDefaultPins();

	// Set text after PostPlacedNewNode, which resets NodeComment to a default value
	CommentNode->NodeComment = CommentText;

	Graph->Modify();
	Graph->AddNode(CommentNode, /*bFromUI=*/true, /*bSelectNewNode=*/false);

	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	return CommentNode;
}

UEdGraphNode_Comment* UBlueprintGraphEditor::AddCommentToNodes(const FString& CommentText, const TArray<UK2Node*>& Nodes, int32 Padding)
{
	if (!IsValid() || Nodes.IsEmpty())
	{
		return nullptr;
	}

	// Compute bounding box from node positions
	int32 MinX = TNumericLimits<int32>::Max();
	int32 MinY = TNumericLimits<int32>::Max();
	int32 MaxX = TNumericLimits<int32>::Min();
	int32 MaxY = TNumericLimits<int32>::Min();

	for (UK2Node* Node : Nodes)
	{
		if (!Node) { continue; }
		const int32 W = Node->NodeWidth  > 0 ? Node->NodeWidth  : static_cast<int32>(UEdGraphSchema_K2::EstimateNodeWidth(Node));
		const int32 H = Node->NodeHeight > 0 ? Node->NodeHeight : static_cast<int32>(UEdGraphSchema_K2::EstimateNodeHeight(Node));
		MinX = FMath::Min(MinX, Node->NodePosX);
		MinY = FMath::Min(MinY, Node->NodePosY);
		MaxX = FMath::Max(MaxX, Node->NodePosX + W);
		MaxY = FMath::Max(MaxY, Node->NodePosY + H);
	}

	if (MinX > MaxX || MinY > MaxY)
	{
		return nullptr;
	}

	const FVector2D Location(MinX - Padding, MinY - Padding);
	const FVector2D Size(MaxX - MinX + Padding * 2, MaxY - MinY + Padding * 2);

	UEdGraphNode_Comment* CommentNode = AddCommentNode(CommentText, Location, Size);
	if (CommentNode)
	{
		for (UK2Node* Node : Nodes)
		{
			if (Node)
			{
				CommentNode->AddNodeUnderComment(Node);
			}
		}
	}
	return CommentNode;
}

TArray<UEdGraphNode_Comment*> UBlueprintGraphEditor::ListCommentNodes() const
{
	TArray<UEdGraphNode_Comment*> Result;
	if (!IsValid()) { return Result; }
	for (UEdGraphNode* Node : Graph->Nodes)
	{
		if (UEdGraphNode_Comment* Comment = Cast<UEdGraphNode_Comment>(Node)) { Result.Add(Comment); }
	}
	return Result;
}

void UBlueprintGraphEditor::RemoveCommentNode(UEdGraphNode_Comment* CommentNode)
{
	if (!IsValid() || !CommentNode) { return; }
	Graph->Modify();
	Graph->RemoveNode(CommentNode);
	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

bool UBlueprintGraphEditor::AddNodePin(UEdGraphNode* Node)
{
	if (!IsValid() || !Node)
	{
		return false;
	}

	// All pin-mutation branches operate on UK2Node-derived types.
	UK2Node* K2Node = Cast<UK2Node>(Node);
	if (!K2Node)
	{
		return false;
	}

	// Honor IK2Node_AddPinInterface::CanAddPin() up front so the bespoke fast paths
	// below cannot bypass any capacity limit a node enforces via the interface.
	IK2Node_AddPinInterface* AddPinNode = Cast<IK2Node_AddPinInterface>(K2Node);
	if (AddPinNode && !AddPinNode->CanAddPin())
	{
		return false;
	}

	if (UK2Node_Switch* SwitchNode = Cast<UK2Node_Switch>(K2Node))
	{
		SwitchNode->Modify();
		SwitchNode->AddPinToSwitchNode();
		MarkBlueprintForPinChange(SwitchNode, Blueprint);
		return true;
	}

	if (UK2Node_FormatText* FormatNode = Cast<UK2Node_FormatText>(K2Node))
	{
		FormatNode->Modify();
		FormatNode->AddArgumentPin();
		MarkBlueprintForPinChange(FormatNode, Blueprint);
		return true;
	}

	// Interface is intentionally last because some classes only partially implement the interface
	// and have bespoke methods for add or remove.
	if (AddPinNode)
	{
		K2Node->Modify();
		AddPinNode->AddInputPin();
		MarkBlueprintForPinChange(K2Node, Blueprint);
		return true;
	}

	return false;
}

bool UBlueprintGraphEditor::RemoveNodePin(UEdGraphNode* Node, const FBlueprintGraphPin& Pin)
{
	if (!IsValid() || !Node)
	{
		return false;
	}

	UEdGraphPin* NativePin = UBlueprintGraphPinLibrary::GetNativePinSafe(Pin);
	if (!NativePin || NativePin->GetOwningNodeUnchecked() != Node)
	{
		return false;
	}

	// All pin-mutation branches operate on UK2Node-derived types.
	UK2Node* K2Node = Cast<UK2Node>(Node);
	if (!K2Node)
	{
		return false;
	}

	// Honor IK2Node_AddPinInterface::CanRemovePin() up front so the bespoke fast paths
	// below cannot bypass any restriction a node enforces via the interface.
	IK2Node_AddPinInterface* AddPinNode = Cast<IK2Node_AddPinInterface>(K2Node);
	if (AddPinNode && !AddPinNode->CanRemovePin(NativePin))
	{
		return false;
	}

	if (UK2Node_Select* SelectNode = Cast<UK2Node_Select>(K2Node))
	{
		if (!SelectNode->CanRemoveOptionPinToNode())
		{
			return false;
		}
		// RemoveOptionPinToNode() always removes the trailing option pin, so reject
		// any other pin to avoid silently removing a different pin than requested.
		TArray<UEdGraphPin*> OptionPins;
		SelectNode->GetOptionPins(OptionPins);
		if (OptionPins.Num() == 0 || OptionPins.Last() != NativePin)
		{
			return false;
		}
		const int32 PinCountBefore = SelectNode->Pins.Num();
		SelectNode->Modify();
		NativePin->Modify();
		SelectNode->RemoveOptionPinToNode();
		const bool bRemoved = SelectNode->Pins.Num() < PinCountBefore;
		if (bRemoved)
		{
			MarkBlueprintForPinChange(SelectNode, Blueprint);
		}
		return bRemoved;
	}

	if (UK2Node_FormatText* FormatNode = Cast<UK2Node_FormatText>(K2Node))
	{
		const UEdGraphPin* FormatPin = FormatNode->GetFormatPin();
		int32 ArgIndex = INDEX_NONE;
		int32 Count = 0;
		for (UEdGraphPin* ExistingPin : FormatNode->Pins)
		{
			if (ExistingPin->Direction == EGPD_Input && ExistingPin != FormatPin)
			{
				if (ExistingPin == NativePin)
				{
					ArgIndex = Count;
					break;
				}
				++Count;
			}
		}
		if (ArgIndex == INDEX_NONE)
		{
			return false;
		}
		const int32 PinCountBefore = FormatNode->Pins.Num();
		FormatNode->Modify();
		NativePin->Modify();
		FormatNode->RemoveArgument(ArgIndex);
		const bool bRemoved = FormatNode->Pins.Num() < PinCountBefore;
		if (bRemoved)
		{
			MarkBlueprintForPinChange(FormatNode, Blueprint);
		}
		return bRemoved;
	}

	if (UK2Node_ExecutionSequence* SequenceNode = Cast<UK2Node_ExecutionSequence>(K2Node))
	{
		if (!SequenceNode->CanRemoveExecutionPin())
		{
			return false;
		}
		const int32 PinCountBefore = SequenceNode->Pins.Num();
		SequenceNode->Modify();
		NativePin->Modify();
		SequenceNode->RemovePinFromExecutionNode(NativePin);
		const bool bRemoved = SequenceNode->Pins.Num() < PinCountBefore;
		if (bRemoved)
		{
			MarkBlueprintForPinChange(SequenceNode, Blueprint);
		}
		return bRemoved;
	}

	if (UK2Node_Switch* SwitchNode = Cast<UK2Node_Switch>(K2Node))
	{
		// UK2Node_SwitchEnum hides pins in place (sets bAdvancedView and breaks
		// links) rather than shrinking Pins; snapshot all three state forms so we
		// report success based on whichever the subclass actually mutated.
		const int32 PinCountBefore = SwitchNode->Pins.Num();
		const bool bWasVisible = !NativePin->bAdvancedView;
		const int32 LinkCountBefore = NativePin->LinkedTo.Num();
		SwitchNode->Modify();
		NativePin->Modify();
		SwitchNode->RemovePinFromSwitchNode(NativePin);
		const bool bRemoved =
			SwitchNode->Pins.Num() < PinCountBefore
			|| (bWasVisible && NativePin->bAdvancedView)
			|| NativePin->LinkedTo.Num() < LinkCountBefore;
		if (bRemoved)
		{
			MarkBlueprintForPinChange(SwitchNode, Blueprint);
		}
		return bRemoved;
	}

	// Interface is intentionally last because some classes only partially implement the interface
	// and have bespoke methods for add or remove.
	if (AddPinNode)
	{
		K2Node->Modify();
		NativePin->Modify();
		AddPinNode->RemoveInputPin(NativePin);
		MarkBlueprintForPinChange(K2Node, Blueprint);
		return true;
	}

	return false;
}

void UBlueprintGraphEditor::RemoveNodes(const TArray<UK2Node*>& Nodes)
{
	if(!IsValid())
	{
		return;
	}

	bool bNeedToModifyStructurally = false;
	for(UK2Node* Node : Nodes)
	{
		if(!Node)
		{
			continue;
		}

		if (Node->CanUserDeleteNode())
		{
			if (Node->NodeCausesStructuralBlueprintChange())
			{
				bNeedToModifyStructurally = true;
			}
				
			FBlueprintEditorUtils::RemoveNode(Blueprint, Node, true);
		}
	}

	if (bNeedToModifyStructurally)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
	else
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

FBlueprintGraphPin UBlueprintGraphEditor::AddGraphInputParameter(FName InputName, const FEdGraphPinType& PinType, const FString& Value)
{
	if(!IsValid())
	{
		return FBlueprintGraphPin{};
	}

	UK2Node_FunctionEntry* Entry = GetFunctionEntryNode();
	if(!Entry)
	{
		return FBlueprintGraphPin{};
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	Entry->Modify();
	UEdGraphPin* Pin = Entry->CreateUserDefinedPin(InputName,
		PinType != FEdGraphPinType() ? PinType : UBlueprintEditorLibrary::GetBasicTypeByName(UEdGraphSchema_K2::PC_Boolean),
		EGPD_Output);
	if(Pin)
	{
		Schema->TrySetDefaultValue(*Pin, Value, false);
		return UBlueprintGraphPinLibrary::FromNativePin(Pin);
	}

	return FBlueprintGraphPin{};
}

bool UBlueprintGraphEditor::IsValid() const
{
	return ::IsValid(Graph) && ::IsValid(Blueprint);
}

FVector2D UBlueprintGraphEditor::GetNodeLocation()
{
	const double GRID_SIZE = 120.0;
	const double X_LIMIT = 2000.0;

	SpawnLocation += FVector2D(GRID_SIZE, 0.0);
	if(SpawnLocation.X > X_LIMIT)
	{
		SpawnLocation.X = 0.0;
		SpawnLocation.Y += GRID_SIZE;
	}
	return SpawnLocation;
}

UK2Node_FunctionEntry* UBlueprintGraphEditor::GetFunctionEntryNode() const
{
	if(!IsValid())
	{
		return nullptr;
	}

	TArray<UK2Node_FunctionEntry*> EntryNodes;
	Graph->GetNodesOfClass(EntryNodes);
	if(EntryNodes.Num() != 1)
	{
		return nullptr;
	}

	return EntryNodes[0];
}

void UBlueprintGraphEditor::HandleNodeSpawned(UK2Node* Result)
{
	UE_LOGFMT(LogBlueprint, Display, "Node {node_path} spawn at ({pos_x},{pox_y})", Result->GetPathName(), Result->NodePosX, Result->NodePosY);

	if (Result->GetSchema()->MarkBlueprintDirtyFromNewNode(Blueprint, Result))
	{
		return;
	}

	UK2Node* K2Node = Cast<UK2Node>(Result);
	if (Result->NodeCausesStructuralBlueprintChange())
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
	else
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

const FProperty* UBlueprintGraphEditor::FindParameterByName(FName Name) const
{
	const UFunction* SkeletonFunction = GetSkeletonFunction();
	if(!SkeletonFunction)
	{
		return nullptr;
	}

	for (TFieldIterator<FProperty> ParamIt(SkeletonFunction); ParamIt && (ParamIt->PropertyFlags & CPF_Parm); ++ParamIt)
	{
		FProperty* Param = *ParamIt;
		const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_ReturnParm) && (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm));
		if (bIsFunctionInput && Param->GetFName() == Name)
		{
			return *ParamIt;
		}
	}

	return nullptr;
}

const FProperty* UBlueprintGraphEditor::FindLocalByName(FName Name) const
{
	const UFunction* SkeletonFunction = GetSkeletonFunction();
	if(!SkeletonFunction)
	{
		return nullptr;
	}

	TFieldIterator<FProperty> ParamIt(SkeletonFunction);
	// skip params:
	while(ParamIt && (ParamIt->PropertyFlags & CPF_Parm))
	{
		++ParamIt;
	}
	while(ParamIt)
	{
		if((*ParamIt)->GetFName() == Name)
		{
			return *ParamIt;
		}
		++ParamIt;
	}

	return nullptr;
}

const UFunction* UBlueprintGraphEditor::GetSkeletonFunction() const
{
	return Blueprint->SkeletonGeneratedClass ? 
		Blueprint->SkeletonGeneratedClass->FindFunctionByName(Graph->GetFName()) : 
		nullptr;
}

UFunction* UBlueprintGraphEditor::GetSkeletonFunction()
{
	const UBlueprintGraphEditor* Self = this;
	return const_cast<UFunction*>(Self->GetSkeletonFunction());
}

const UClass* UBlueprintGraphEditor::GetClassFromPath(const FString& ClassPath)
{
	TSoftObjectPtr<UClass> ClassSoftPtr = TSoftObjectPtr<UClass>(FSoftObjectPath(ClassPath));
	const UClass* Class = ClassSoftPtr.LoadSynchronous();
	if(Class)
	{
		return Class;
	}
	return nullptr;
}

void UBlueprintGraphEditor::SetFunctionFlagImpl(UFunction* Function, UK2Node_FunctionEntry* EntryNode, EFunctionFlags Flag, bool bIsSet)
{
	if (!ensure(EntryNode) || !ensure(Function))
	{
		return;
	}

	EntryNode->Modify();
	Function->Modify();

	if(bIsSet)
	{
		Function->FunctionFlags |= Flag;
		EntryNode->SetExtraFlags(EntryNode->GetExtraFlags() | Flag);
	}
	else
	{
		Function->FunctionFlags &= ~Flag;
		EntryNode->SetExtraFlags(EntryNode->GetExtraFlags() & ~Flag);
	}
}

void UBlueprintGraphEditor::ListAvailableNodes(TArray<FString>& Result, const TArray<FBlueprintGraphPin>& ContextPins) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlueprintGraphEditor::ListAvailableNodes);

	Result.Reset();
	if (!IsValid())
	{
		return;
	}
	FBlueprintActionFilter Filter;
	FBlueprintActionContext& FilterContext = Filter.Context;
	for (const FBlueprintGraphPin& ContextPin : ContextPins)
	{
		if (UEdGraphPin* NativePin = UBlueprintGraphPinLibrary::GetNativePinSafe(ContextPin))
		{
			FilterContext.Pins.Add(NativePin);
		}
	}

	FilterContext.Graphs.Add(Graph);
	FilterContext.Blueprints.Add(Blueprint);

	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	FBlueprintActionDatabase::FActionRegistry const& ActionRegistry = ActionDatabase.GetAllActions();

	for (auto Iterator(ActionRegistry.CreateConstIterator()); Iterator; ++Iterator)
	{
		const FObjectKey& ObjKey = Iterator->Key;
		const FBlueprintActionDatabase::FActionList& ActionList = Iterator->Value;

		if (UObject* ActionObject = ObjKey.ResolveObjectPtr())
		{
			for (UBlueprintNodeSpawner const* NodeSpawner : ActionList)
			{
				FBlueprintActionInfo BlueprintAction(ActionObject, NodeSpawner);

				bool bPassedFilter = !Filter.IsFiltered(BlueprintAction);

				if (bPassedFilter)
				{
					UBlueprintNodeSpawner const* Action = BlueprintAction.NodeSpawner;
					if (!Action)
					{
						continue;
					}
					FBlueprintActionUiSpec UISpec = Action->GetUiSpec(FilterContext, BlueprintAction.GetBindings());
					FString NameAndCategory = FString::Printf(TEXT("%s|%s"),
						*UISpec.Category.ToString().Replace(TEXT(" "), TEXT("")),
						*UISpec.MenuName.ToString().Replace(TEXT(" "), TEXT("")));
					Result.Add(NameAndCategory);	
				}
			}
		}
		else
		{
			// Remove this (invalid) entry on the next tick.
			ActionDatabase.DeferredRemoveEntry(ObjKey);
		}
	}
}

UEdGraphNode* UBlueprintGraphEditor::CreateNodeFromName(const FString& NodeWithCategory, FVector2D const Location, const TArray<FBlueprintGraphPin>& ContextPins, UClass* DeclaringClass)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlueprintGraphEditor::ListAvailableNodes);

	if (!IsValid())
	{
		return nullptr;
	}
	FBlueprintActionFilter Filter;
	FBlueprintActionContext& FilterContext = Filter.Context;
	FilterContext.Graphs.Add(Graph);
	FilterContext.Blueprints.Add(Blueprint);
	for (const FBlueprintGraphPin& ContextPin : ContextPins)
	{
		if (UEdGraphPin* NativePin = UBlueprintGraphPinLibrary::GetNativePinSafe(ContextPin))
		{
			FilterContext.Pins.Add(NativePin);
		}
	}

	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	FBlueprintActionDatabase::FActionRegistry const& ActionRegistry = ActionDatabase.GetAllActions();

	for (auto Iterator(ActionRegistry.CreateConstIterator()); Iterator; ++Iterator)
	{
		const FObjectKey& ObjKey = Iterator->Key;
		const FBlueprintActionDatabase::FActionList& ActionList = Iterator->Value;

		if (UObject* ActionObject = ObjKey.ResolveObjectPtr())
		{
			for (UBlueprintNodeSpawner const* NodeSpawner : ActionList)
			{
				FBlueprintActionInfo BlueprintAction(ActionObject, NodeSpawner);

				bool bPassedFilter = !Filter.IsFiltered(BlueprintAction);

				if (bPassedFilter)
				{
					UBlueprintNodeSpawner const* Action = BlueprintAction.NodeSpawner;
					if (!Action) 
					{
						continue;
					}
					FBlueprintActionUiSpec UISpec = Action->GetUiSpec(FilterContext, BlueprintAction.GetBindings());
					FString NameAndCategory = FString::Printf(TEXT("%s|%s"),
						*UISpec.Category.ToString().Replace(TEXT(" "), TEXT("")),
						*UISpec.MenuName.ToString().Replace(TEXT(" "), TEXT("")));
					if (NodeWithCategory.Equals(NameAndCategory, ESearchCase::CaseSensitive))
					{
						if (DeclaringClass)
						{
							// Blueprint actions are registered on the skeleton class; normalize DeclaringClass
							// to skeleton so the comparison works regardless of which class variant is passed.
							UClass* EffectiveDeclaringClass = FBlueprintEditorUtils::GetSkeletonClass(DeclaringClass);
							if (!EffectiveDeclaringClass)
							{
								EffectiveDeclaringClass = DeclaringClass;
							}
							if (const UFunction* Func = Cast<UFunction>(ActionObject))
							{
								if (Func->GetOwnerClass() != EffectiveDeclaringClass)
								{
									continue;
								}
							}
							else if (const UClass* ActionClass = Cast<UClass>(ActionObject))
							{
								if (ActionClass != DeclaringClass)
								{
									continue;
								}
							}
							else if (const UBlueprint* ActionBlueprint = Cast<UBlueprint>(ActionObject))
							{
								// Blueprint-defined functions and delegates are keyed by UBlueprint* in the
								// action registry. Compare against the skeleton class to handle both variants.
								if (ActionBlueprint->SkeletonGeneratedClass != EffectiveDeclaringClass)
								{
									continue;
								}
							}
							else if (const UBlueprintDelegateNodeSpawner* DelegateSpawner =
								Cast<UBlueprintDelegateNodeSpawner>(Action))
							{
								if (const FMulticastDelegateProperty* DelegateProp =
									DelegateSpawner->GetDelegateProperty())
								{
									if (DelegateProp->GetOwnerClass() != EffectiveDeclaringClass)
									{
										continue;
									}
								}
							}
						}
						UEdGraphNode* SpawnedNode = Action->Invoke(Graph, BlueprintAction.GetBindings(), Location);
						if (!SpawnedNode)
						{
							continue;
						}
						for (UEdGraphPin* Pin : FilterContext.Pins)
						{
							SpawnedNode->AutowireNewNode(Pin);
						}
						if (UK2Node* K2Node = Cast<UK2Node>(SpawnedNode)) 
						{
							HandleNodeSpawned(K2Node);
						}
						return SpawnedNode;
					}
				}
			}
		}
		else
		{
			// Remove this (invalid) entry on the next tick.
			ActionDatabase.DeferredRemoveEntry(ObjKey);
		}
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
