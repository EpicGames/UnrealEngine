// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Templates/SubclassOf.h"
#include "BlueprintEditorLibrary/BlueprintGraphPin.h"

#include "BlueprintGraphEditor.generated.h"

class UEdGraph;
class UBlueprint;

class UK2Node;
class UK2Node_CallFunction;
class UK2Node_FunctionEntry;
class UK2Node_MacroInstance;
class UK2Node_IfThenElse;
class UK2Node_Select;
class UK2Node_FunctionResult;
class UK2Node_VariableGet;
class UK2Node_VariableSet;
class UK2Node_ComponentBoundEvent;
class UEdGraphNode_Comment;
class UBlueprintGraphPin;

/**
 * Object oriented helper for editing graphs in a blueprint object
 * Instantiate the UBlueprintGraphEditor via CreateAndEditFunctionGraph
 * or GetGraphEditor. Static helpers can be found in BlueprintEditorLibrary
 * and pin specific routines are encapsulated by BlueprintGraphPin
 * 
 * Python Example:

import unreal

BlueprintGraphEditor = unreal.BlueprintGraphEditor

print_string = "/Script/Engine.KismetSystemLibrary.PrintString"

def GenHelloWorld(bp):
    editor = BlueprintGraphEditor.create_and_edit_function_graph(bp, "HelloWorld")
    start_pin = editor.find_graph_entry_pin()
    start_pos = start_pin.get_owning_node().get_node_pos()
    print_node = editor.add_call_function_node(print_string)
    print_node.find_input_pin("InString").set_pin_value("Generated Script")
    start_pin.try_create_connection(print_node.find_execute_pin())
    print_node.set_node_pos(unreal.IntPoint(start_pos.x + 160, start_pos.y))

 */
UCLASS(Transient, BlueprintType, HideDropDown)
class UBlueprintGraphEditor : public UObject
{
	GENERATED_BODY()
private:
	// This won't dissuade external NewObject calls, but HideDropDown keeps the editor clients away
	// Please instantiate this object using one of the provided factory functions (e.g. CreateAndEditFunctionGraph)
	UBlueprintGraphEditor(); 

public:
	/** returns a new UBlueprintGraphEditor for a graph in InBlueprint, creating a new graph named FuncName - useful for creating new function graphs */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UBlueprintGraphEditor* CreateAndEditFunctionGraph(UBlueprint* Blueprint, const FString& FuncName = FString(TEXT("NewFunction")));
	
	/** returns a new UBlueprintGraphEditor for a graph in InBlueprint named GraphName - useful for editing graphs that already exist */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UBlueprintGraphEditor* GetGraphEditorByName(UBlueprint* InBlueprint, FName GraphName);

	/** returns a new UBlueprintGraphEditor for the provided graph - useful for editing graphs that already exist */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UBlueprintGraphEditor* GetGraphEditor(UEdGraph* InGraph);

	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UEdGraph* GetGraph() const { return Graph; }

	/**
	 * Lists all nodes in the graph
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void ListAllNodes(TArray<UK2Node*>& OutNodes);
	/**
	 * Lists nodes of the type provided by the Class input
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void ListNodesOfClass(TArray<UK2Node*>& OutNodes, TSubclassOf<UK2Node> Class);
	/**
	 * Lists nodes with EMessageSeverity::Error level compiler messages
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void ListNodesWithErrors(TArray<UK2Node*>& OutNodes);
	/**
	 * Lists nodes with EMessageSeverity::Warning level compiler messages
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void ListNodesWithWarnings(TArray<UK2Node*>& OutNodes);
	/**
	 * Lists nodes with EMessageSeverity::Info level compiler messages
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void ListNodesWithNotes(TArray<UK2Node*>& OutNodes);
	
	/**
	 * List all of the nodes by name that can be added to this graph
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta=(AutoCreateRefTerm = "ContextPins"))
	void ListAvailableNodes(TArray<FString>& Result, const TArray<FBlueprintGraphPin>& ContextPins) const;

	/**
	 * Creates node from Catagory and Name. Example:
	 *			- 'Development|PrintString'
	 *			- 'Utilities|Operators|Add'
	 *			- 'Utilities|FlowControl|Branch'
	 *			- 'Math|Vector|VectorLength'
	 * When multiple actions share the same Category|Name string , pass DeclaringClass to 
	 * select the action belonging to that specific class.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (AutoCreateRefTerm = "ContextPins"))
	UEdGraphNode* CreateNodeFromName(const FString& NodeWithCategory, FVector2D const Location, const TArray<FBlueprintGraphPin>& ContextPins, UClass* DeclaringClass = nullptr);

	/**
	 * Adds a graph input parameter with name InputName, and optional PinType and Value. Returns the added pin.
	 * By default the type will be a boolean as defined by UBlueprintEditorLibrary.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	FBlueprintGraphPin AddGraphInputParameter(FName InputName, const FEdGraphPinType& PinType = FEdGraphPinType(), const FString& Value = FString());
	
	/**
	 * Removes the graph input parameter specified by InputName
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool RemoveGraphInputParameter(FName InputName);
	
	/**
	 * Returns the 'entry' pin for the graph, e.g. the 'then' pin of the entry node
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	FBlueprintGraphPin FindGraphEntryPin() const;

	/**
	 * Returns the event node associated with EventName - aka Member Name
	 * of the event. Names will not resolve against the display name, so e.g. 'Begin Play'
	 * is not a valid member name - instead search for ReceiveBeginPlay, the underlying
	 * BlueprintImplementableEvent
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_Event* FindEventNode(FName EventName) const;
	
	/**
	 * Adds a graph output parameter, returning the UK2Node_FunctionResult that has the new output on it.
	 * By default the type will be a boolean as defined by UBlueprintEditorLibrary.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_FunctionResult* AddGraphOutputParameter(FName OutputName, const FEdGraphPinType& PinType = FEdGraphPinType());
	
	/**
	 * Removes the output parameter specified by OutputName, returns true if there is a variable and it is removed
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool RemoveGraphOutputParameter(FName OutputName);
	
	/**
	 * Lists the names of the local variables in the function.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void ListLocalVariableNames(TArray<FString>& OutLocalVariableNames) const;

	/**
	 * Returns the type of a local variable, unset if no variable is found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	TOptional<FEdGraphPinType> GetLocalVariableType(const FString& LocalVariableName) const;
	
	/**
	 * Returns the default value of a local variable, unset if no variable is found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	TOptional<FString> GetLocalVariableDefaultValue(const FString& LocalVariableName) const;

	/**
	 * Set default value of a local variable. Returns false if the operation fails
	 * either because the value is invalid or there is no variable of the specified name.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool SetLocalVariableDefaultValue(const FString& LocalVariableName, const FString& NewDefaultValue);

	/**
	 * Adds a local variable, use BlueprintEditorLibrary to specify a type. A default value can optionally be provided.
	 * By default the type will be a boolean as defined by UBlueprintEditorLibrary.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool AddLocalVariable(FName LocalName, const FEdGraphPinType& PinType = FEdGraphPinType(), const FString& Value = FString());
	/**
	 * Removes the local variable with the corresponding name, returns true if there is a variable and it is removed
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool RemoveLocalVariable(FName LocalName);
	
	/**
	 * Adds a member variable, use BlueprintEditorLibrary to specify a type. A default value can optionally be provided.
	 * By default the type will be a boolean as defined by UBlueprintEditorLibrary.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool AddMemberVariable(FName MemberName,  const FEdGraphPinType& PinType = FEdGraphPinType(), const FString& Value = FString());
	/**
	 * Removes the member variable with the corresponding name, returns true if there is a variable and it is removed
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool RemoveMemberVariable(FName MemberName);

	/**
	 * Sets the graph's FUNC_BlueprintPure flag
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetIsPureFunction(bool bIsPure = true);
	/**
	 * Sets the graph's callability in editor
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetIsCallInEditorFunction(bool bCallInEditor = true);
	/**
	 * Sets the graph's visibility to public
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetFunctionIsPublic();
	/**
	 * Sets the graph's visibility to protected
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetFunctionIsProtected();
	/**
	 * Sets the graph's visibility to private
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetFunctionIsPrivate();
	/**
	 * Sets the graph's FUNC_Const flag
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetIsConstFunction(bool bIsConst = true);
	/**
	 * Sets the graph's FUNC_Exec flag
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetIsExecFunction(bool bIsExec = true);
	/**
	 * Sets the graph's thread safety status
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetIsThreadSafeFunction(bool bIsThreadSafe = true);
	/**
	 * Sets the graph's construction script safety status
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetIsUnsafeDuringActorConstructionFunction(bool bIsUnsafeDuringActorConstruction = true);
	/**
	 * Sets the graph's deprecation status
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetIsDeprecatedFunction(bool bIsDeprecated = true);
	/**
	 * Sets the graph's deprecation message
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void SetDeprecationMessageOnFunction(FText Message);

	/**
	 * Creates a UK2Node_CallFunction in the current graph invoking the function specified by FunctionPath
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_CallFunction* AddCallFunctionNode(const FString& FunctionPath);
	/**
	 * Creates a UK2Node_MacroInstance in the current graph for the provided MacroPath
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_MacroInstance* AddMacroNode(const FString& MacroPath);
	
	/**
	 * Creates a UK2Node_IfThenElse in the current graph
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_IfThenElse* AddBranchNode();
	/**
	 * Creates a UK2Node_FunctionResult in the current graph
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_FunctionResult* AddReturnNode();

	/**
	 * Creates a UK2Node_VariableGet for the member variable identified by MemberName
	 * by default this will use the current class, provided a ClassPath to work
	 * with a different class
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_VariableGet* AddGetMemberVariableNode(FName MemberName, const FString& ClassPath = FString());
	/**
	 * Creates a UK2Node_VariableSet for the member variable identified by MemberName
	 * by default this will use the current class, provided a ClassPath to work
	 * with a different class
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_VariableSet* AddSetMemberVariableNode(FName MemberName, const FString& ClassPath = FString());
	/**
	 * Creates a UK2Node_VariableGet for the provided local variable identified by LocalName
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_VariableGet* AddGetLocalVariableNode(FName LocalName);
	/**
	 * Creates a UK2Node_VariableSet for the provided local variable identified by LocalName
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_VariableSet* AddSetLocalVariableNode(FName LocalName);

	/**
	 * Creates a UK2Node_CustomEvent node if the current graph is an event graph, using the provided EventName
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_CustomEvent* AddCustomEventNode(const FString& EventName);

	/**
	 * Creates an event node in this event graph whose pins match the signature of the named dispatcher.
	 * The current graph must be an Event Graph and the Blueprint must have a valid SkeletonGeneratedClass
	 * (i.e. it must have been compiled at least once). Returns nullptr if the graph is not an Event Graph,
	 * the skeleton class is absent, or no dispatcher named DispatcherName exists on the Blueprint.
	 * When DeclaringClass is provided the dispatcher is only accepted if it is declared on exactly that
	 * class, not merely inherited through it - use this to disambiguate between sibling classes that
	 * expose a dispatcher under the same name.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_Event* AddDispatcherEventNode(FName DispatcherName, UClass* DeclaringClass = nullptr);

	/**
	 * Replaces a node's baked-in class reference from OldClass to NewClass and reconstructs
	 * the node so its pins reflect the new type. Handles the following node kinds:
	 *   - UK2Node_DynamicCast     (TargetType)
	 *   - UK2Node_CallFunction    (FunctionReference)
	 *   - UK2Node_Event           (EventReference)
	 *   - UK2Node_BaseMCDelegate  (DelegateReference - covers all multicast delegate nodes)
	 * Returns true if the node was retargeted or already references NewClass.
	 * Returns false if the node type is unsupported or its current class reference does not match OldClass.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool RetargetNodeClass(UEdGraphNode* Node, UClass* OldClass, UClass* NewClass);

	/**
	 * Returns the names of all bindable delegate events on the given component.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	TArray<FName> ListComponentEvents(UActorComponent* Component) const;

	/**
	 * Creates a UK2Node_ComponentBoundEvent in this event graph for the given component
	 * and delegate event name. Returns the existing node if one already exists.
	 * The graph must be an event graph.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UK2Node_ComponentBoundEvent* AddComponentBoundEventNode(
		UActorComponent* Component, FName EventName);

	/**
	 * Creates a comment node in the current graph at the given location with the given text and size.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UEdGraphNode_Comment* AddCommentNode(const FString& CommentText, FVector2D Location, FVector2D Size = FVector2D(400, 200));

	/**
	 * Creates a comment node that wraps the given nodes with the given padding.
	 * The comment bounds are computed from the node positions plus padding.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	UEdGraphNode_Comment* AddCommentToNodes(const FString& CommentText, const TArray<UK2Node*>& Nodes, int32 Padding = 50);

	/**
	 * Returns all comment nodes in the current graph.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	TArray<UEdGraphNode_Comment*> ListCommentNodes() const;

	/**
	 * Removes a comment node from the current graph.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void RemoveCommentNode(UEdGraphNode_Comment* CommentNode);

	/**
	 * Adds a pin to a node that supports dynamic pin addition, such as a Switch node,
	 * Sequence node, commutative binary operator (Add, Multiply), or Make Array node.
	 * Returns true on success, false if the node type does not support adding pins.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool AddNodePin(UEdGraphNode* Node);

	/**
	 * Removes a specific pin from a node that supports dynamic pin removal, such as a Sequence
	 * node, Switch node, commutative binary operator (Add, Multiply), or Make Array node.
	 * Returns true on success, false if the node type does not support removing pins or the
	 * pin cannot be removed (e.g. minimum pin count reached).
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	bool RemoveNodePin(UEdGraphNode* Node, const FBlueprintGraphPin& Pin);

	/**
	 * Removes (aka deletes) the provided nodes from the current graph
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	void RemoveNodes(const TArray<UK2Node*>& Nodes);
private:
	bool IsValid() const;
	FVector2D GetNodeLocation();
	UK2Node_FunctionEntry* GetFunctionEntryNode() const;
	void HandleNodeSpawned(UK2Node* Result);
	const FProperty* FindParameterByName(FName Name) const;
	const FProperty* FindLocalByName(FName Name) const;
	const UFunction* GetSkeletonFunction() const;
	UFunction* GetSkeletonFunction();
	static const UClass* GetClassFromPath(const FString& ClassPath);
	static void SetFunctionFlagImpl(UFunction* Function, UK2Node_FunctionEntry* EntryNode, EFunctionFlags Flag, bool bIsSet);

	UPROPERTY()
	FVector2D SpawnLocation;

	UPROPERTY()
	TObjectPtr<UBlueprint> Blueprint;

	UPROPERTY()
	TObjectPtr<UEdGraph> Graph;
};
