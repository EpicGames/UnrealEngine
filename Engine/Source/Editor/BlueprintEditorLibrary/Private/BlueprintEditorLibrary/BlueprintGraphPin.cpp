// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEditorLibrary/BlueprintGraphPin.h"

#include "BlueprintEditorLibrary.h"
#include "EdGraphSchema_K2.h"
#include "K2Node.h"
#include "Logging/StructuredLog.h"

bool FBlueprintGraphPin::operator==(const FBlueprintGraphPin&) const = default;
bool FBlueprintGraphPin::operator!=(const FBlueprintGraphPin&) const = default;

FBlueprintGraphPin UBlueprintGraphPinLibrary::FromNativePin(UEdGraphPin* Pin)
{
	if(!Pin)
	{
		return FBlueprintGraphPin{};
	}

	UK2Node* OwningNode = Cast<UK2Node>(Pin->GetOwningNode());
	if(!OwningNode)
	{
		return FBlueprintGraphPin{};
	}

	FBlueprintGraphPin ScriptPin;
	ScriptPin.Node = OwningNode;
	ScriptPin.PinDirection = Pin->Direction;
	ScriptPin.PinName = Pin->PinName;

	int32 LogicalIndex = 0;
	for(UEdGraphPin* NodePin : OwningNode->Pins)
	{
		if(NodePin->bHidden)
		{
			continue;
		}
		if(NodePin->Direction == Pin->Direction)
		{
			if(NodePin == Pin)
			{
				ScriptPin.PinIndex = LogicalIndex;
				break;
			}
			LogicalIndex++;
		}
	}

	return ScriptPin;
}

bool UBlueprintGraphPinLibrary::IsValid(const FBlueprintGraphPin& Pin)
{
	if( Pin == FBlueprintGraphPin() )
	{
		return false;
	}

	return true;
}

bool UBlueprintGraphPinLibrary::IsSameNativePin(const FBlueprintGraphPin& Pin, const FBlueprintGraphPin& Other)
{
	const UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	const UEdGraphPin* OtherNativePin = GetNativePinSafe(Other);
	if(!NativePin || !OtherNativePin)
	{
		return false;
	}

	return NativePin == OtherNativePin;
}

bool UBlueprintGraphPinLibrary::SetPinValue(const FBlueprintGraphPin& Pin, const FString& Value)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	if(UEdGraphPin* NativePin = GetNativePinSafe(Pin))
	{
		Schema->TrySetDefaultValue(*NativePin, Value, false);
		// Reset default values
		if (!Schema->IsPinDefaultValid(NativePin, NativePin->DefaultValue, NativePin->DefaultObject, NativePin->DefaultTextValue).IsEmpty())
		{
			Schema->ResetPinToAutogeneratedDefaultValue(NativePin, false);
			UE_LOGFMT(LogBlueprint, Warning, "Failed to set default value {Value} on {PinName}", Value, NativePin->PinName);
			return false;
		}
		return true;
	}
	return false;
}

bool UBlueprintGraphPinLibrary::TryCreateConnection(const FBlueprintGraphPin& Pin, const FBlueprintGraphPin& Other)
{
	UEdGraphPin* SelfNativePin = GetNativePinSafe(Pin);
	UEdGraphPin* OtherNativePin = GetNativePinSafe(Other);
	if(!SelfNativePin || !OtherNativePin)
	{
		return false;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	return Schema->TryCreateConnection(SelfNativePin, OtherNativePin);
}

bool UBlueprintGraphPinLibrary::CanCreateConnection(const FBlueprintGraphPin& Pin, const FBlueprintGraphPin& Other)
{
	const UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	const UEdGraphPin* OtherNativePin = GetNativePinSafe(Other);
	if(!NativePin || !OtherNativePin)
	{
		return false;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	return Schema->CanCreateConnection(NativePin, OtherNativePin).Response != CONNECT_RESPONSE_DISALLOW;
}

bool UBlueprintGraphPinLibrary::BreakPinLinks(const FBlueprintGraphPin& Pin)
{
	UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	if(!NativePin)
	{
		return false;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	Schema->BreakPinLinks(*NativePin, true);
	return true;
}

bool UBlueprintGraphPinLibrary::BreakSinglePinLink(const FBlueprintGraphPin& Pin, const FBlueprintGraphPin& Other)
{
	UEdGraphPin* SelfNativePin = GetNativePinSafe(Pin);
	UEdGraphPin* OtherNativePin = GetNativePinSafe(Other);
	if(!SelfNativePin || !OtherNativePin)
	{
		return false;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	Schema->BreakSinglePinLink(SelfNativePin, OtherNativePin);
	return true;
}

FName UBlueprintGraphPinLibrary::GetPinName(const FBlueprintGraphPin& Pin)
{
	UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	if(!NativePin)
	{
		return NAME_None;
	}

	return NativePin->PinName;
}

TEnumAsByte<EEdGraphPinDirection> UBlueprintGraphPinLibrary::GetPinDirection(const FBlueprintGraphPin& Pin)
{
	return Pin.PinDirection;
}

FString UBlueprintGraphPinLibrary::GetPinTypeAsJsonSchema(const FBlueprintGraphPin& Pin)
{
	UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	if(!NativePin)
	{
		return TEXT("");
	}

	UK2Node* OwningNode = Cast<UK2Node>(NativePin->GetOwningNode());
	if(!OwningNode)
	{
		return TEXT("");
	}

	return UBlueprintEditorLibrary::PinTypeToJsonSchema(NativePin->PinType, OwningNode->GetBlueprintClassFromNode());
}

FText UBlueprintGraphPinLibrary::GetPinTypeDisplayString(const FBlueprintGraphPin& Pin)
{
	UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	if(!NativePin)
	{
		return FText();
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	return Schema->TypeToText(NativePin->PinType);
}

FEdGraphPinType UBlueprintGraphPinLibrary::GetPinType(const FBlueprintGraphPin& Pin)
{
	UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	if(!NativePin)
	{
		return FEdGraphPinType();
	}

	return NativePin->PinType;
}

FString UBlueprintGraphPinLibrary::GetPinValue(const FBlueprintGraphPin& Pin)
{
	UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	if(!NativePin)
	{
		return TEXT("");
	}

	return NativePin->GetDefaultAsString();
}

UK2Node* UBlueprintGraphPinLibrary::GetOwningNode(const FBlueprintGraphPin& Pin)
{
	return Pin.Node;
}

TArray<FBlueprintGraphPin> UBlueprintGraphPinLibrary::ListConnectedPins(const FBlueprintGraphPin& Pin)
{
	TArray<FBlueprintGraphPin> Result;
	UEdGraphPin* NativePin = GetNativePinSafe(Pin);
	if(!NativePin)
	{
		return Result;
	}

	for(UEdGraphPin* LinkedToPin : NativePin->LinkedTo)
	{
		FBlueprintGraphPin WrappedPin = FromNativePin(LinkedToPin);
		if(::IsValid(WrappedPin.Node))
		{
			Result.Add(WrappedPin);
		}
	}
	return Result;
}

UEdGraphPin* UBlueprintGraphPinLibrary::GetNativePinSafe(const FBlueprintGraphPin& Pin)
{
	if(!::IsValid(Pin.Node))
	{
		return nullptr;
	}

	// Collect visible pins in the requested direction so we can index into them
	// and perform an outward search if needed.
	TArray<UEdGraphPin*, TInlineAllocator<16>> DirectionalPins;
	for(UEdGraphPin* NativePin : Pin.Node->Pins)
	{
		if(!NativePin->bHidden && NativePin->Direction == Pin.PinDirection)
		{
			DirectionalPins.Add(NativePin);
		}
	}

	// Fast path: index and name both match
	if(DirectionalPins.IsValidIndex(Pin.PinIndex) && DirectionalPins[Pin.PinIndex]->PinName == Pin.PinName)
	{
		return DirectionalPins[Pin.PinIndex];
	}

	// The pin at the expected index has a different name — the graph has been
	// modified and indices have drifted. Search outward from PinIndex to find
	// the nearest pin that still has the correct name.
	for(int32 Distance = 1; Distance < DirectionalPins.Num(); ++Distance)
	{
		const int32 Forward = Pin.PinIndex + Distance;
		if(DirectionalPins.IsValidIndex(Forward) && DirectionalPins[Forward]->PinName == Pin.PinName)
		{
			return DirectionalPins[Forward];
		}

		const int32 Backward = Pin.PinIndex - Distance;
		if(DirectionalPins.IsValidIndex(Backward) && DirectionalPins[Backward]->PinName == Pin.PinName)
		{
			return DirectionalPins[Backward];
		}
	}

	return nullptr;
}
