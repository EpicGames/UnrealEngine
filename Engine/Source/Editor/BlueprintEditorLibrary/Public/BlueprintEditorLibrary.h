// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorLibrary/BlueprintGraphPin.h"
#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Logging/LogMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintEditorLibrary.generated.h"

#define UE_API BLUEPRINTEDITORLIBRARY_API

class FProperty;
class UBlueprint;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UEdGraphNode_Comment;
class UK2Node;
class UK2Node_CreateDelegate;
class UK2Node_Event;
class UObject;
struct FFrame;

DECLARE_LOG_CATEGORY_EXTERN(LogBlueprintEditorLib, Warning, All);

/**
 * Network replication mode for a Blueprint variable.
 * Blueprint-exposed proxy for EVariableReplication (BlueprintDetailsCustomization.h:74).
 */
UENUM(BlueprintType)
enum class EBlueprintVariableReplication : uint8
{
	/** Not replicated. */
	None       UMETA(DisplayName = "None"),
	/** Replicated from server to client. */
	Replicated UMETA(DisplayName = "Replicated"),
	/** Replicated with a notification function called on clients when a new value arrives. */
	RepNotify  UMETA(DisplayName = "RepNotify"),
};

/**
 * The results of comparing an assets save version to another
 */
UENUM(BlueprintType)
enum class EAssetSaveVersionComparisonResults : uint8
{
	// The comparison could not be completed
	InvalidComparison,
	// The asset save version is identical to what it is being compared to
	Identical,
	// The asset save version is newer than what it is being compared to
	Newer,
	// The asset save version is older than what it is being compared to
	Older
};

/** Per-item info for a function or event visible on a Blueprint. */
USTRUCT(BlueprintType)
struct FBlueprintFunctionInfo
{
	GENERATED_BODY()

	/** The function or event's identifier — the name a caller would use to refer to it. */
	UPROPERTY(BlueprintReadOnly, Category = "Blueprint Editor")
	FName Name;

	/** Short human-readable description, e.g. the tooltip shown in the editor. Empty if none. */
	UPROPERTY(BlueprintReadOnly, Category = "Blueprint Editor")
	FText Description;

	/** True if this function/event already has a graph or node on the Blueprint; false if it is only available to implement or override. */
	UPROPERTY(BlueprintReadOnly, Category = "Blueprint Editor")
	bool bIsImplemented = false;
};

UCLASS(MinimalAPI)
class UBlueprintEditorLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
public:
	/**
	* Replace any references of variables with the OldVarName to references of those with the NewVarName if possible
	*
	* @param Blueprint		Blueprint to replace the variable references on
	* @param OldVarName		The variable you want replaced
	* @param NewVarName		The new variable that will be used in the old one's place
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API void ReplaceVariableReferences(UBlueprint* Blueprint, const FName OldVarName, const FName NewVarName);

	/**
	* Finds the event graph of the given blueprint. Null if it doesn't have one. This will only return
	* the primary event graph of the blueprint (the graph named "EventGraph").
	*
	* @param Blueprint		Blueprint to search for the event graph on
	*
	* @return UEdGraph*		Event graph of the blueprint if it has one, null if it doesn't have one
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API UEdGraph* FindEventGraph(UBlueprint* Blueprint);

	/**
	 * Compares the given assets save version to the VersionToCheck. 
	 * 
	 * @param Asset				The asset which you would like to check the SavedByEngineVersion of.
	 * 
	 * @param VersionToCheck	String representation of the engine version to compare against. For example, "5.6.0-37518009+++UE5+Main"
	 *							@see GetSavedByEngineVersion and GetCurrentEngineVersion
	 * 
	 * @param Result			The outcome of the version comparison
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools",  Meta = (ExpandEnumAsExecs = "Result"))
	static UE_API void CompareAssetSaveVersionTo(const UObject* Asset, const FString& VersionToCheck, EAssetSaveVersionComparisonResults& Result);

	/**
	 * Compares the given soft object's save version to the VersionToCheck. This will read the packages file header
	 * 
	 * @param ObjectToCheck		Soft object pointer to the object whose save version you would like to compare.
	 * 
	 * @param VersionToCheck	String representation of the engine version to compare against.  For example, "5.6.0-37518009+++UE5+Main"
	 *							@see GetSavedByEngineVersion and GetCurrentEngineVersion
	 * 
	 * @param Result			The outcome of the version comparison
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools",  Meta = (ExpandEnumAsExecs = "Result"))
	static UE_API void CompareSoftObjectSaveVersionTo(const TSoftObjectPtr<UObject> ObjectToCheck,  const FString& VersionToCheck, EAssetSaveVersionComparisonResults& Result);

	/**
	 * Returns a string representation of the engine version which the given asset was saved with.
	 * 
	 * @see FLinker::Summary::SavedByEngineVersion
	 * @see FPackageFileSummary
	 * 
	 * @param Asset The asset to check the saved by engine version of.
	 * 
	 * @return	String representation of the engine version which this asset was saved with. "INVALID" if none. 
	 *			For example: "5.6.0-37518009+++UE5+Main"
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Blueprint Upgrade Tools")
	static UE_API FString GetSavedByEngineVersion(const UObject* Asset);

	/**
	 * Returns a string which represents the current engine version (FEngineVersion::Current())
	 * 
	 * For example: "5.6.0-37518009+++UE5+Main"
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Blueprint Upgrade Tools")
	static UE_API FString GetCurrentEngineVersion();

	/**
	* Finds the graph with the given name on the blueprint. Null if it doesn't have one. 
	*
	* @param Blueprint		Blueprint to search
	* @param GraphName		The name of the graph to search for 
	*
	* @return UEdGraph*		Pointer to the graph with the given name, null if not found
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API UEdGraph* FindGraph(UBlueprint* Blueprint, FName GraphName);

	/**
	* Lists all of the graphs that a Blueprint contains.
	*
	* @param Blueprint			Blueprint to enumerate	
	*
	* @return TArray<FName>		A list of the graphs in the blueprint.
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API TArray<UEdGraph*> ListGraphs(UBlueprint* Blueprint);

	/**
	* Replace any old operator nodes (float + float, vector + float, int + vector, etc)
	* with the newer Promotable Operator version of the node. Preserve any connections the
	* original node had to the newer version of the node. 
	*
	* @param Blueprint	Blueprint to upgrade
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API void UpgradeOperatorNodes(UBlueprint* Blueprint);

	/**
	* Compiles the given blueprint. 
	*
	* @param Blueprint	Blueprint to compile
	* 
	* @return True if compilation succeeded with no errors
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API bool CompileBlueprint(UBlueprint* Blueprint);

	/**
	* Adds a function to the given blueprint
	*
	* @param Blueprint	The blueprint to add the function to
	* @param FuncName	Name of the function to add
	*
	* @return UEdGraph*
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API UEdGraph* AddFunctionGraph(UBlueprint* Blueprint, const FString& FuncName = FString(TEXT("NewFunction")));

	/**
	 * Lists all non-event functions visible on this Blueprint: locally defined function
	 * graphs, overridable parent-class functions, and functions declared on interfaces
	 * (both explicitly implemented and inherited through a natively-implemented interface).
	 * Each entry carries an is_implemented flag indicating whether a function graph for
	 * that name exists on the Blueprint.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API TArray<FBlueprintFunctionInfo> ListFunctions(UBlueprint* Blueprint);

	/**
	 * Lists all events visible on this Blueprint: locally defined custom events, overridable
	 * parent-class events, and event-shape interface members. Each entry carries an
	 * is_implemented flag indicating whether the event node has been placed on the Blueprint.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API TArray<FBlueprintFunctionInfo> ListEvents(UBlueprint* Blueprint);

	/**
	 * Creates a function-graph override of an inherited function. The graph's terminator
	 * nodes inherit the parent's signature and a CallParentFunction node is emitted.
	 *
	 * If a function graph with this name already exists it is returned unchanged. Fails if
	 * the function is already overridden as an event node (remove that node first).
	 *
	 * @param Blueprint		The blueprint to add the override to
	 * @param FunctionName	Name of an inherited overridable function.
	 * @return				The override function graph, or nullptr if the function
	 *						cannot be overridden on this Blueprint.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API UEdGraph* AddFunctionOverride(UBlueprint* Blueprint, FName FunctionName);

	/**
	 * Creates an event-override node in the Blueprint's event graph for an inherited
	 * event-shape function.
	 *
	 * If an event node for this function already exists on the Blueprint it is returned
	 * unchanged. Fails (returns nullptr) when the function is not an event-shape
	 * overridable on this Blueprint, when no event graph exists, or when the Blueprint
	 * is invalid.
	 *
	 * @param Blueprint		The blueprint to add the event override to
	 * @param EventName		Name of an inherited overridable event-shape function.
	 * @param Position		Position for the new node in the event graph.
	 * @return				The event-override node, or nullptr if it could not be created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API UK2Node_Event* AddEventOverride(UBlueprint* Blueprint, FName EventName,
		FIntPoint Position);

	/**
	* Deletes the function of the given name on this blueprint. Does NOT replace function call sites.
	*
	* @param Blueprint		The blueprint to remove the function from
	* @param FuncName		The name of the function to remove
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API void RemoveFunctionGraph(UBlueprint* Blueprint, FName FuncName);

	/**
	* Remove any nodes in this blueprint that have no connections made to them.
	*
	* @param Blueprint		The blueprint to remove the nodes from
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API void RemoveUnusedNodes(UBlueprint* Blueprint);

	/** 
	* Removes the given graph from the blueprint if possible 
	* 
	* @param Blueprint	The blueprint the graph will be removed from
	* @param Graph		The graph to remove
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API void RemoveGraph(UBlueprint* Blueprint, UEdGraph* Graph);

	/**
	* Attempts to rename the given graph with a new name
	*
	* @param Graph			The graph to rename
	* @param NewNameStr		The new name of the graph
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API void RenameGraph(UEdGraph* Graph, const FString& NewNameStr = FString(TEXT("NewGraph")));

	/**
	* Finds the UBlueprint associated with the object, locally searching the object graph for a
	* UBlueprint associated with an asset object. If the Object is a UBlueprint this function will 
	* perform a simple cast. Note that the blueprint object itself is editor only and not present 
	* in cooked assets.
	*
	* @param Object			The object we need to get the UBlueprint from
	*
	* @return UBlueprint*	The blueprint associated with the given object, nullptr if the object has 
	* no associated blueprint or the blueprint is in another package
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools", meta = (Keywords = "cast"))
	static UE_API UBlueprint* GetBlueprintAsset(UObject* Object);
	
	/**
	 * Looks up the UBlueprint that generated the provided class, if any. Provides a 'true' exec pin
	 * to execute if there is a valid blueprint associated with the Class.
	 * 
	 * @param Class						The class to look up the blueprint for
	 * @param bDoesClassHaveBlueprint	Whether the provided class had a blueprint
	 * 
	 * @return							The blueprint that generated the class, nullptr if the UClass
										is native or otherwise cooked
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools", meta = (ExpandBoolAsExecs = "bDoesClassHaveBlueprint"))
	static UE_API UBlueprint* GetBlueprintForClass(UClass* Class, bool& bDoesClassHaveBlueprint);

	/** Attempt to refresh any open blueprint editors for the given asset */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API void RefreshOpenEditorsForBlueprint(const UBlueprint* BP);

	/** Refresh any open blueprint editors */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API void RefreshAllOpenBlueprintEditors();

	/**
	* Returns the parent class of the given Blueprint.
	*
	* @param Blueprint			Blueprint to query
	* @return					The parent class of the Blueprint, or null if the Blueprint is invalid
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Blueprint Editor", meta=(ScriptMethod))
	static UE_API UClass* GetBlueprintParentClass(const UBlueprint* Blueprint);

	/**
	* Attempts to reparent the given blueprint to the new chosen parent class.
	*
	* @param Blueprint			Blueprint that you would like to reparent
	* @param NewParentClass		The new parent class to use
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API void ReparentBlueprint(UBlueprint* Blueprint, UClass* NewParentClass);

	/**
	* Gathers any unused blueprint variables and populates the given array of FPropertys
	*
	* @param Blueprint			The blueprint to check
	* @param OutProperties		Out array of unused FProperty*'s
	*
	* @return					True if variables were checked on this blueprint, false otherwise.
	*/
	static UE_API bool GatherUnusedVariables(const UBlueprint* Blueprint, TArray<FProperty*>& OutProperties);

	/**
	* Deletes any unused blueprint created variables the given blueprint.
	* An Unused variable is any BP variable that is not referenced in any 
	* blueprint graphs
	* 
	* @param Blueprint			Blueprint that you would like to remove variables from
	*
	* @return					Number of variables removed
	*/
	UFUNCTION(BlueprintCallable, Category = "Blueprint Upgrade Tools")
	static UE_API int32 RemoveUnusedVariables(UBlueprint* Blueprint);

	/**
	 * Gets the class generated when this blueprint is compiled
	 *
	 * @param BlueprintObj		The blueprint object
	 * @return UClass*			The generated class
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API UClass* GeneratedClass(UBlueprint* BlueprintObj);

	/** Changes the specified member variable's type - inherited variables must have their type changed at the point of declaration */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API void ChangeMemberVariableType(UBlueprint* Blueprint, const FName& VariableName, const FEdGraphPinType& NewType);

	/**
	 * Sets "Expose On Spawn" to true/false on a Blueprint variable
	 *
	 * @param Blueprint			The blueprint object
	 * @param VariableName		The variable name
	 * @param bExposeOnSpawn	Set to true to expose on spawn
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API void SetBlueprintVariableExposeOnSpawn(UBlueprint* Blueprint, const FName& VariableName, bool bExposeOnSpawn);

	/**
	 * Sets "Expose To Cinematics" to true/false on a Blueprint variable
	 *
	 * @param Blueprint				The blueprint object
	 * @param VariableName			The variable name
	 * @param bExposeToCinematics	Set to true to expose to cinematics
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API void SetBlueprintVariableExposeToCinematics(UBlueprint* Blueprint, const FName& VariableName, bool bExposeToCinematics);

	/**
	 * Sets "Instance Editable" to true/false on a Blueprint variable
	 *
	 * @param Blueprint				The blueprint object
	 * @param VariableName			The variable name
	 * @param bInstanceEditable		Toggle InstanceEditable
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API void SetBlueprintVariableInstanceEditable(UBlueprint* Blueprint, const FName& VariableName, bool bInstanceEditable);

	/**
	 * Gets the replication mode of a Blueprint variable.
	 *
	 * @param Blueprint		The blueprint object
	 * @param VariableName	The variable name
	 * @return				The replication mode: None, Replicated, or RepNotify
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API EBlueprintVariableReplication GetBlueprintVariableReplication(
		UBlueprint* Blueprint, const FName& VariableName);

	/**
	 * Sets the replication mode on a Blueprint variable.
	 * RepNotify will auto-create an OnRep_ function if one does not already exist.
	 *
	 * @param Blueprint		The blueprint object
	 * @param VariableName	The variable name
	 * @param Replication	The replication mode: None, Replicated, or RepNotify
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API void SetBlueprintVariableReplication(UBlueprint* Blueprint, const FName& VariableName,
		EBlueprintVariableReplication Replication);

	/**
	 * Gets the user-defined category of a Blueprint member variable. Categories are
	 * used to group variables in the My Blueprint panel.
	 *
	 * @param Blueprint		The blueprint object
	 * @param VariableName	The variable name
	 * @return				The category text. Empty if the variable was not found or
	 *						has no explicit category override (defaults to the
	 *						Blueprint's name in the UI).
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API FText GetBlueprintVariableCategory(UBlueprint* Blueprint, const FName& VariableName);

	/**
	 * Sets the user-defined category on a Blueprint member variable. Categories are
	 * used to group variables in the My Blueprint panel.
	 * Note: Will not change the category for variables defined via native classes.
	 *
	 * @param Blueprint		The blueprint object
	 * @param VariableName	The variable name
	 * @param NewCategory	The new category text. Pass empty to reset to the default
	 *						(the Blueprint's name).
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting", meta = (ScriptMethod))
	static UE_API void SetBlueprintVariableCategory(UBlueprint* Blueprint, const FName& VariableName, const FText& NewCategory);

	/**
	 * Returns the comment text of a comment node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API FString GetCommentText(const UEdGraphNode_Comment* CommentNode);

	/**
	 * Sets the comment text of a comment node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API void SetCommentText(UEdGraphNode_Comment* CommentNode, const FString& NewText);

	/**
	 * Gets the background color of a comment node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API FLinearColor GetCommentColor(const UEdGraphNode_Comment* CommentNode);

	/**
	 * Sets the background color of a comment node.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API void SetCommentColor(UEdGraphNode_Comment* CommentNode, FLinearColor Color);

	/**
	 * Returns the K2 nodes contained within the given comment node.
	 * Only UK2Node-derived nodes are returned; nested comments and other non-K2 node
	 * types are excluded.
	 * Membership is explicitly maintained when comments are created programmatically.
	 * Note: if the comment is manually moved or resized in the editor, membership is
	 * rebuilt spatially and may diverge from the programmatic assignment.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API TArray<UK2Node*> GetNodesInComment(const UEdGraphNode_Comment* CommentNode);

	/**
	 * Returns the size (width, height) of a node as stored in the graph data.
	 * Note: sizes are only populated after the graph has been rendered in the editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API FVector2D GetNodeSize(const UEdGraphNode* Node);

	/**
	 * Creates a blueprint based on a specific parent, honoring registered custom blueprint types
	 *
	 * @param AssetPath				The full path that the asset should be created with
	 * @param ParentClass			The parent class that the blueprint should be based on
	 */
	 UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	 static UE_API UBlueprint* CreateBlueprintAssetWithParent(const FString& AssetPath, UClass* ParentClass);

	/**
	 * Lists the names of the editable graphs in the BP
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static void ListGraphNames(const UBlueprint* Blueprint, TArray<FName>& OutGraphNames);

	/**
	 * Lists the names of the member variables in the BP. Inherited members will be prefixed with the full path of their declaring class.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static void ListMemberVariableNames(const UBlueprint* Blueprint, TArray<FString>& OutMemberVariableNames, bool bIncludeInheritedMembers = true);
	
	/**
	 * Returns the type of a member variable, unset if no variable is found. Inherited variables can be requested using a fully qualified name to avoid ambiguities.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static TOptional<FEdGraphPinType> GetMemberVariableType(const UBlueprint* Blueprint, const FString& MemberVariableName);

	/**
	  * Adds a member variable to the specified blueprint inferring the type from a provided value.
	  * 
	  * @return	true if it succeeds, false if it fails.
	  */
	UFUNCTION(BlueprintCallable, CustomThunk, Category = "Blueprint Editor", meta=(CustomStructureParam="DefaultValue"))
	static UE_API bool AddMemberVariableWithValue(UBlueprint* Blueprint, FName MemberName, const int32& DefaultValue);
	static UE_API bool Generic_AddMemberVariableWithValue(UBlueprint* Blueprint, FName MemberName, const uint8* DefaultValuePtr, const FProperty* DefaultValueProp);
	DECLARE_FUNCTION(execAddMemberVariableWithValue);
	
	/**
	  * Adds a member variable to the specified blueprint with the specified type.
	  *
	  * @return	true if it succeeds, false if it fails.
	  */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API bool AddMemberVariable(UBlueprint* Blueprint, FName MemberName, const FEdGraphPinType& VariableType);

	/**
	 * Creates a new event dispatcher on the Blueprint with the given name.
	 * Returns false if the name is already in use or the Blueprint is invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API bool AddEventDispatcher(UBlueprint* Blueprint, FName Name);

	/**
	 * Removes the event dispatcher with the given name from the Blueprint.
	 * Returns false if no dispatcher with that name exists or the Blueprint is invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API bool RemoveEventDispatcher(UBlueprint* Blueprint, FName Name);

	/**
	 * Returns the names of all event dispatchers defined on the Blueprint.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API TArray<FName> ListEventDispatchers(const UBlueprint* Blueprint);

	/**
	 * Adds a parameter to an event dispatcher's signature.
	 * Returns false if the dispatcher does not exist or the parameter name is already in use.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API bool AddEventDispatcherParameter(UBlueprint* Blueprint, FName DispatcherName,
		FName ParamName, const FEdGraphPinType& ParamType);

	/**
	 * Removes a parameter from an event dispatcher's signature.
	 * Returns false if the dispatcher or parameter does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API bool RemoveEventDispatcherParameter(UBlueprint* Blueprint, FName DispatcherName,
		FName ParamName);
	
	/** @return a pintype for 'int', 'byte', 'bool', 'real', 'name', 'string' or 'text' - returns 'int' type if invalid type is provided */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API FEdGraphPinType GetBasicTypeByName(FName TypeName);
	
	/** @return a pintype for the provided struct - returns 'int' type if invalid struct is provided */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API FEdGraphPinType GetStructType(const UScriptStruct* StructType);
	
	/** @return a class reference pintype for the provided class - returns 'int' type if invalid class is provided */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API FEdGraphPinType GetClassReferenceType(const UClass* ClassType);
	
	/** @return a object reference pintype for the provided class - returns 'int' type if invalid object type is provided */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API FEdGraphPinType GetObjectReferenceType(const UClass* ObjectType);
	
	/** @return a array of ContainedType type - returns 'int' type if invalid type is provided */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API FEdGraphPinType GetArrayType(const FEdGraphPinType& ContainedType);
	
	/** @return a set of ContainedType type - returns 'int' type if invalid type is provided */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API FEdGraphPinType GetSetType(const FEdGraphPinType& ContainedType);
	
	/** @return a map of KeyType to ValueType type - returns 'int' type if invalid type is provided */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor")
	static UE_API FEdGraphPinType GetMapType(const FEdGraphPinType& KeyType,const FEdGraphPinType& ValueType);
	
	/**
	 * @ A json schema string describing the type
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UE_API FString PinTypeToJsonSchema(const FEdGraphPinType& PinType, const UClass* SelfContext);

	/// Begin UK2Node extension/script methods
	
	/**
	 * Returns all visible pins on this node, optionally discriminated by direction
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static TArray<FBlueprintGraphPin> ListAllPins(const UK2Node* Node, TEnumAsByte<EEdGraphPinDirection> InDirection = EEdGraphPinDirection::EGPD_MAX);

	/**
	 * Returns all visible input pins on this node
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static TArray<FBlueprintGraphPin> ListInputPins(const UK2Node* Node);

	/**
	 * Returns all visible output pins on this node
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static TArray<FBlueprintGraphPin> ListOutputPins(const UK2Node* Node);

	/**
	 * Returns an input pin specified by index and, optionally, type
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindInputPinByIndex(const UK2Node* Node, int32 Index, const FEdGraphPinType& Type = FEdGraphPinType());

	/**
	 * Returns an input pin specified by name and, optionally, type
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindInputPin(const UK2Node* Node, FName PinName, const FEdGraphPinType& Type = FEdGraphPinType());

	/**
	 * Returns an output pin specified by index and, optionally, type
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindOutputPinByIndex(const UK2Node* Node, int32 Index, const FEdGraphPinType& Type = FEdGraphPinType());

	/**
	 * Returns an output pin specified by name and, optionally, type
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindOutputPin(const UK2Node* Node, FName PinName, const FEdGraphPinType& Type = FEdGraphPinType());

	/**
	 * Returns the 'execute' or 'do' pin associated with this node, if any
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindExecutePin(const UK2Node* Node);

	/**
	 * Returns the 'then' pin associated with this node, if any
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static UE_API FBlueprintGraphPin FindThenPin(const UK2Node* Node);

	/**
	 * Returns the self pin associated with this node, if any
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindSelfPin(const UK2Node* Node);

	/**
	 * Returns the single data output pin associated with this node, returning default pin if there are multiple or no output(s)
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindResultPin(const UK2Node* Node);

	/**
	 * Returns the single data input pin associated with this node, returning default pin if there are multiple or no input(s)
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindDataInputPin(const UK2Node* Node);
	
	/**
	 * Sets this node's visual position in the graph
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static void SetNodePos(UK2Node* Node, FIntPoint pos);
	
	/**
	 * Returns this node's visual position in the graph
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FIntPoint GetNodePos(const UK2Node* Node);
	
	/**
	 * Returns this node's title, as used when the node is displayed in a list
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FString GetNodeTitle(const UK2Node* Node);

	/**
	 * Returns this node's menu category
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FString GetNodeCategory(const UK2Node* Node);

	/// end UK2Node extension/script methods

	/// begin UK2Node_CreateDelegate extension/script methods

	/**
	 * Sets the selected function on a Create Event node (the dropdown value).
	 * Use ListCompatibleFunctionsForDelegate to find valid function names.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static void SetCreateDelegateFunction(UK2Node_CreateDelegate* Node, FName FunctionName);

	/**
	 * Returns the selected function name on a Create Event node (the dropdown value).
	 * Returns NAME_None if no function is selected.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FName GetCreateDelegateFunction(const UK2Node_CreateDelegate* Node);

	/**
	 * Lists functions compatible with the delegate signature of a Create Event node.
	 * Returns an empty array if the node's delegate output pin is not yet connected.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static TArray<FName> ListCompatibleFunctionsForDelegate(const UK2Node_CreateDelegate* Node);

	/// end UK2Node_CreateDelegate extension/script methods

	/// begin UK2Node_IfThenElse textension/script methods:

	/** Finds the Else pin on this node */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindElsePin(const UK2Node_IfThenElse* Node);

	/** Finds the Condition pin on this node */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Graph Editing", meta = (ScriptMethod))
	static FBlueprintGraphPin FindConditionPin(const UK2Node_IfThenElse* Node);

	/// end UK2Node_IfThenElse extension/script methods
};

#undef UE_API
