// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "EdGraph/EdGraphNode.h"

#include "BlueprintGraphPin.generated.h"

#define UE_API BLUEPRINTEDITORLIBRARY_API

class UEdGraph;
class UBlueprint;

class UK2Node;

/** 
	A value type representing a blueprint graph pin (underlying native type: UEdGraphPin). Use IsValid()
	to detect whether it represents a real pin value (e.g. after finding a pin on a node).
*/
USTRUCT(BlueprintType)
struct FBlueprintGraphPin
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UK2Node> Node = nullptr;

	UPROPERTY()
	TEnumAsByte<EEdGraphPinDirection> PinDirection = EEdGraphPinDirection::EGPD_MAX;

	UPROPERTY()
	FName PinName;

	UPROPERTY()
	int32 PinIndex = 0;

	UE_API bool operator==(const FBlueprintGraphPin&) const;
	UE_API bool operator!=(const FBlueprintGraphPin&) const;
};

/** 
 * API for FBlueprintGraphPin
 */
UCLASS(Transient, HideDropDown)
class UBlueprintGraphPinLibrary : public UObject
{
	GENERATED_BODY()
public:
	/** Creates a FBlueprintGraphPin representing the underlying native Pin */
	static FBlueprintGraphPin FromNativePin(UEdGraphPin* Pin);
	
	/** @return true if the pin has a valid underlying pin */
	UFUNCTION(BlueprintPure, Category = "Blueprint Editor", meta = (ScriptMethod))
	static bool IsValid(const FBlueprintGraphPin& Pin);

	/** 
	 * Because these pin objects are proxies for the underlying native pin, this function
	 * should be used for establishing whether to FBlueprintGraphPin values refer to the 
	 * same pin on a node
	 * 
	 * @return true if the Other pin and this Pin are the same pin
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static bool IsSameNativePin(const FBlueprintGraphPin& Pin, const FBlueprintGraphPin& Other);

	/**
	 * Sets the literal value associated with this pin, returning false if the value is not valid
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static bool SetPinValue(const FBlueprintGraphPin& Pin, const FString& Value);
	
	/**
	 * Tries to create a connection between this pin and Other, returning false if the connection is not allowed
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static bool TryCreateConnection(const FBlueprintGraphPin& Pin, const FBlueprintGraphPin& Other);

	/**
	 * Returns true if this and Other pin can be connected. Performing connection may result in implicitly created conversion nodes.
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static bool CanCreateConnection(const FBlueprintGraphPin& Pin, const FBlueprintGraphPin& Other);

	/**
	 * Breaks all pin links associated with this pin
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static bool BreakPinLinks(const FBlueprintGraphPin& Pin);
	
	/**
	 * Breaks a pin link between this pin and Other
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static bool BreakSinglePinLink(const FBlueprintGraphPin& Pin, const FBlueprintGraphPin& Other);
	
	/**
	 * Returns the low level name of the pin
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static FName GetPinName(const FBlueprintGraphPin& Pin);
	
	/**
	 * Returns the direction of the pin (either EGPD_Output or EGPD_Input)
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static TEnumAsByte<EEdGraphPinDirection> GetPinDirection(const FBlueprintGraphPin& Pin);

	/**
	 * Returns a json schema encoded description of the pin's type:
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static FString GetPinTypeAsJsonSchema(const FBlueprintGraphPin& Pin);

	/**
	 * Returns the string used to describe the pin type in the blueprint UI
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static FText GetPinTypeDisplayString(const FBlueprintGraphPin& Pin);

	/**
	 * Returns the pin's type, FEdGraphPinType is mostly opaque to script but can
	 * be used e.g. to create variables of the same type
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static FEdGraphPinType GetPinType(const FBlueprintGraphPin& Pin);

	/**
	 * Returns the literal value associated with the pin. Empty string for pins 
	 * that have nothing set or are connected to some other pin:
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static FString GetPinValue(const FBlueprintGraphPin& Pin);
	
	/**
	 * Returns the pin's node
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static UK2Node* GetOwningNode(const FBlueprintGraphPin& Pin);

	/**
	 * Returns a list of pins this pin is connected to, note that these pin objects are 
	 * proxies and so comparison to other pin objects must be done via IsSameNativePin
	 */
	UFUNCTION(BlueprintCallable, Category = "Blueprint Editor", meta = (ScriptMethod))
	static TArray<FBlueprintGraphPin> ListConnectedPins(const FBlueprintGraphPin& Pin);

	static UEdGraphPin* GetNativePinSafe(const FBlueprintGraphPin& Pin);
};

#undef UE_API
